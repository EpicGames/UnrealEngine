// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class GameplayDebugger : ModuleRules
	{
		public GameplayDebugger(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"InputCore",
					"RenderCore",
					"Slate",
					"DeveloperSettings",
				});
			
			if (Target.Configuration != UnrealTargetConfiguration.Shipping)
			{
				PrivateDependencyModuleNames.Add("DrawPrimitiveDebugger");
			}

			PrivateIncludePathModuleNames.Add("Settings");

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[] {
						"LevelEditor",
						"UnrealEd",
					});
			}

			SetupIrisSupport(Target);

			if (Target.bUseGameplayDebuggerCore)
			{
				PrecompileForTargets = PrecompileTargetsType.Any;
			}

			if (Target.bUseGameplayDebugger || Target.bUseGameplayDebuggerCore)
			{
				PublicDefinitions.Add("WITH_GAMEPLAY_DEBUGGER_CORE=1");
				PublicDefinitions.Add("WITH_GAMEPLAY_DEBUGGER=" + (Target.bUseGameplayDebugger ? 1 : 0));
				if (Target.bUseGameplayDebugger || (Target.bUseGameplayDebuggerCore && Target.Configuration != UnrealTargetConfiguration.Shipping))
				{
					PublicDefinitions.Add("WITH_GAMEPLAY_DEBUGGER_MENU=1");
				}
				else
				{
					PublicDefinitions.Add("WITH_GAMEPLAY_DEBUGGER_MENU=0");
				}
			}
			else
			{
				PublicDefinitions.Add("WITH_GAMEPLAY_DEBUGGER_CORE=0");
				PublicDefinitions.Add("WITH_GAMEPLAY_DEBUGGER=0");
				PublicDefinitions.Add("WITH_GAMEPLAY_DEBUGGER_MENU=0");
			}
		}
	}
}
