// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayDebuggerAddonBase.h"
#include "GameplayDebuggerCategoryReplicator.h"
#include "GameFramework/PlayerController.h"
#if WITH_EDITOR
#include "Editor/EditorEngine.h"
#endif

AActor* FGameplayDebuggerAddonBase::FindLocalDebugActor() const
{
	AGameplayDebuggerCategoryReplicator* RepOwnerOb = RepOwner.Get();
	return RepOwnerOb ? RepOwnerOb->GetDebugActor() : nullptr;
}

AGameplayDebuggerCategoryReplicator* FGameplayDebuggerAddonBase::GetReplicator() const
{
	return RepOwner.Get();
}

FString FGameplayDebuggerAddonBase::GetInputHandlerDescription(int32 HandlerId) const
{
	return InputHandlers.IsValidIndex(HandlerId) ? InputHandlers[HandlerId].ToString() : FString();
}

UWorld* FGameplayDebuggerAddonBase::GetWorldFromReplicator() const
{
	AGameplayDebuggerCategoryReplicator* Replicator = RepOwner.Get();
	return Replicator ? Replicator->GetWorld() : nullptr;
}

UWorld* FGameplayDebuggerAddonBase::GetDataWorld(const APlayerController* OwnerPC, const AActor* DebugActor) const
{
	// We're trying OwnerPC first because that's the main owner for what's being displayed by gameplay debugger
	// but it editor-only mode there are no player controllers, so we try the DebugActor, which can be picked
	// also in the editor-mode by selecting an actor on the level.
	// finally, DebugActor can be null as well, when no actor is selected, when we fetch the world which the
	// AGameplayDebuggerCategoryReplicator instance owning this debugger addon belongs to.
	UWorld* World = OwnerPC ? OwnerPC->GetWorld() : nullptr;
	if (World == nullptr && DebugActor != nullptr)
	{
		World = DebugActor->GetWorld();
	}
	return World ? World : GetWorldFromReplicator();
}

void FGameplayDebuggerAddonBase::OnGameplayDebuggerActivated()
{
	// empty in base class
}

void FGameplayDebuggerAddonBase::OnGameplayDebuggerDeactivated()
{
	// empty in base class
}

bool FGameplayDebuggerAddonBase::IsSimulateInEditor()
{
#if WITH_EDITOR
	extern UNREALED_API UEditorEngine* GEditor;
	if (GEditor)
	{
		return GEditor->IsSimulateInEditorInProgress();
	}
#endif
	return false;
}
