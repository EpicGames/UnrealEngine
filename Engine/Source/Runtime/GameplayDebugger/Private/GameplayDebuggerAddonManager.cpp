// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayDebuggerAddonManager.h"
#include "Engine/World.h"
#include "GameplayDebuggerCategoryReplicator.h"
#include "GameplayDebuggerExtension.h"
#include "GameplayDebuggerConfig.h"

FGameplayDebuggerAddonManager::FGameplayDebuggerAddonManager()
{
	NumVisibleCategories = 0;
}

void FGameplayDebuggerAddonManager::RegisterCategory(FName CategoryName, IGameplayDebugger::FOnGetCategory MakeInstanceDelegate, EGameplayDebuggerCategoryState CategoryState, int32 SlotIdx)
{
	FGameplayDebuggerCategoryInfo NewInfo;
	NewInfo.MakeInstanceDelegate = MakeInstanceDelegate;
	NewInfo.DefaultCategoryState = CategoryState;

	UGameplayDebuggerConfig* MutableToolConfig = UGameplayDebuggerConfig::StaticClass()->GetDefaultObject<UGameplayDebuggerConfig>();
	uint8 NewCategoryState = (uint8)CategoryState;
	MutableToolConfig->UpdateCategoryConfig(CategoryName, SlotIdx, NewCategoryState);

	NewInfo.CategoryState = (EGameplayDebuggerCategoryState)NewCategoryState;
	NewInfo.SlotIdx = SlotIdx;

	CategoryMap.Add(CategoryName, NewInfo);

	// create and destroy single instance to handle input configurators
	FGameplayDebuggerInputHandlerConfig::CurrentCategoryName = CategoryName;
	TSharedRef<FGameplayDebuggerCategory> DummyRef = MakeInstanceDelegate.Execute();
	FGameplayDebuggerInputHandlerConfig::CurrentCategoryName = NAME_None;
}

void FGameplayDebuggerAddonManager::UnregisterCategory(FName CategoryName)
{
	CategoryMap.Remove(CategoryName);
}

void FGameplayDebuggerAddonManager::NotifyCategoriesChanged()
{
	struct FSlotInfo
	{
		FName CategoryName;
		int32 CategoryId;
		int32 SlotIdx;

		bool operator<(const FSlotInfo& Other) const { return (SlotIdx == Other.SlotIdx) ? CategoryName.LexicalLess(Other.CategoryName) : (SlotIdx < Other.SlotIdx); }
	};

	TArray<FSlotInfo> AssignList;
	TSet<int32> OccupiedSlots;
	int32 CategoryId = 0;
	for (auto It : CategoryMap)
	{
		if (It.Value.CategoryState == EGameplayDebuggerCategoryState::Hidden)
		{
			continue;
		}

		const int32 SanitizedSlotIdx = (It.Value.SlotIdx < 0) ? INDEX_NONE : FMath::Min(100, It.Value.SlotIdx);

		FSlotInfo SlotInfo;
		SlotInfo.CategoryId = CategoryId;
		SlotInfo.CategoryName = It.Key;
		SlotInfo.SlotIdx = SanitizedSlotIdx;
		CategoryId++;

		AssignList.Add(SlotInfo);

		if (SanitizedSlotIdx >= 0)
		{
			OccupiedSlots.Add(SanitizedSlotIdx);
		}
	}

	NumVisibleCategories = AssignList.Num();
	AssignList.Sort();

	int32 MaxSlotIdx = 0;
	for (int32 Idx = 0; Idx < AssignList.Num(); Idx++)
	{
		FSlotInfo& SlotInfo = AssignList[Idx];
		if (SlotInfo.SlotIdx == INDEX_NONE)
		{
			int32 FreeSlotIdx = 0;
			while (OccupiedSlots.Contains(FreeSlotIdx))
			{
				FreeSlotIdx++;
			}

			SlotInfo.SlotIdx = FreeSlotIdx;
			OccupiedSlots.Add(FreeSlotIdx);
		}

		MaxSlotIdx = FMath::Max(MaxSlotIdx, SlotInfo.SlotIdx);
	}

	SlotMap.Reset();
	SlotNames.Reset();
	SlotMap.AddDefaulted(MaxSlotIdx + 1);
	SlotNames.AddDefaulted(MaxSlotIdx + 1);

	for (int32 Idx = 0; Idx < AssignList.Num(); Idx++)
	{
		FSlotInfo& SlotInfo = AssignList[Idx];

		if (SlotNames[SlotInfo.SlotIdx].Len())
		{
			SlotNames[SlotInfo.SlotIdx] += TEXT('+');
		}

		SlotNames[SlotInfo.SlotIdx] += SlotInfo.CategoryName.ToString();
		SlotMap[SlotInfo.SlotIdx].Add(SlotInfo.CategoryId);
	}

	OnCategoriesChanged.Broadcast();
}

void FGameplayDebuggerAddonManager::CreateCategories(AGameplayDebuggerCategoryReplicator& Owner, TArray<TSharedRef<FGameplayDebuggerCategory> >& CategoryObjects)
{
	UWorld* World = Owner.GetWorld();
	check(World);
	const ENetMode NetMode = World->GetNetMode();
	const bool bHasAuthority = FGameplayDebuggerUtils::IsAuthority(World);
	const bool bIsSimulate = FGameplayDebuggerAddonBase::IsSimulateInEditor();

	TArray<TSharedRef<FGameplayDebuggerCategory> > UnsortedCategories;
	for (auto It : CategoryMap)
	{
		FGameplayDebuggerInputHandlerConfig::CurrentCategoryName = It.Key;
		if (It.Value.CategoryState == EGameplayDebuggerCategoryState::Hidden)
		{
			continue;
		}

		TSharedRef<FGameplayDebuggerCategory> CategoryObjectRef = It.Value.MakeInstanceDelegate.Execute();
		FGameplayDebuggerCategory& CategoryObject = CategoryObjectRef.Get();
		CategoryObject.RepOwner = &Owner;
		CategoryObject.CategoryId = CategoryObjects.Num();
		CategoryObject.CategoryName = It.Key;
		CategoryObject.bHasAuthority = bHasAuthority;
		CategoryObject.bIsLocal = Owner.IsLocal();
		CategoryObject.bIsEnabled =
			(It.Value.CategoryState == EGameplayDebuggerCategoryState::EnabledInGameAndSimulate) ||
			(It.Value.CategoryState == EGameplayDebuggerCategoryState::EnabledInGame && !bIsSimulate) ||
			(It.Value.CategoryState == EGameplayDebuggerCategoryState::EnabledInSimulate && bIsSimulate);

		UnsortedCategories.Add(CategoryObjectRef);
	}

	FGameplayDebuggerInputHandlerConfig::CurrentCategoryName = NAME_None;

	// sort by slots for drawing order
	CategoryObjects.Reset();
	for (int32 SlotIdx = 0; SlotIdx < SlotMap.Num(); SlotIdx++)
	{
		for (int32 Idx = 0; Idx < SlotMap[SlotIdx].Num(); Idx++)
		{
			const int32 CategoryId = SlotMap[SlotIdx][Idx];
			if (ensureMsgf(UnsortedCategories.IsValidIndex(CategoryId), TEXT("Invalid Category Id from SlotMap - Could not add the Gameplay Debugger Category")))
			{
				CategoryObjects.Add(UnsortedCategories[CategoryId]);
			}
		}
	}
}

void FGameplayDebuggerAddonManager::RegisterExtension(FName ExtensionName, IGameplayDebugger::FOnGetExtension MakeInstanceDelegate)
{
	FGameplayDebuggerExtensionInfo NewInfo;
	NewInfo.MakeInstanceDelegate = MakeInstanceDelegate;
	NewInfo.bDefaultEnabled = true;

	uint8 UseExtension = NewInfo.bDefaultEnabled ? 1 : 0;
	UGameplayDebuggerConfig* MutableToolConfig = UGameplayDebuggerConfig::StaticClass()->GetDefaultObject<UGameplayDebuggerConfig>();
	MutableToolConfig->UpdateExtensionConfig(ExtensionName, UseExtension);

	NewInfo.bEnabled = UseExtension > 0;

	ExtensionMap.Add(ExtensionName, NewInfo);

	// create and destroy single instance to handle input configurators
	FGameplayDebuggerInputHandlerConfig::CurrentExtensionName = ExtensionName;
	TSharedRef<FGameplayDebuggerExtension> DummyRef = MakeInstanceDelegate.Execute();
	FGameplayDebuggerInputHandlerConfig::CurrentExtensionName = NAME_None;
}

void FGameplayDebuggerAddonManager::UnregisterExtension(FName ExtensionName)
{
	ExtensionMap.Remove(ExtensionName);
}

void FGameplayDebuggerAddonManager::NotifyExtensionsChanged()
{
	OnExtensionsChanged.Broadcast();
}

void FGameplayDebuggerAddonManager::CreateExtensions(AGameplayDebuggerCategoryReplicator& Replicator, TArray<TSharedRef<FGameplayDebuggerExtension> >& ExtensionObjects)
{
	ExtensionObjects.Reset();
	for (auto It : ExtensionMap)
	{
		if (It.Value.bEnabled)
		{
			FGameplayDebuggerInputHandlerConfig::CurrentExtensionName = It.Key;

			TSharedRef<FGameplayDebuggerExtension> ExtensionObjectRef = It.Value.MakeInstanceDelegate.Execute();
			FGameplayDebuggerExtension& ExtensionObject = ExtensionObjectRef.Get();
			ExtensionObject.RepOwner = &Replicator;

			ExtensionObjects.Add(ExtensionObjectRef);
		}
	}

	FGameplayDebuggerInputHandlerConfig::CurrentExtensionName = NAME_None;
}

void FGameplayDebuggerAddonManager::UpdateFromConfig()
{
	UGameplayDebuggerConfig* ToolConfig = UGameplayDebuggerConfig::StaticClass()->GetDefaultObject<UGameplayDebuggerConfig>();
	if (ToolConfig == nullptr)
	{
		return;
	}

	bool bCategoriesChanged = false;
	for (auto& It : CategoryMap)
	{
		for (int32 Idx = 0; Idx < ToolConfig->Categories.Num(); Idx++)
		{
			const FGameplayDebuggerCategoryConfig& ConfigData = ToolConfig->Categories[Idx];
			if (*ConfigData.CategoryName == It.Key)
			{
				const bool bDefaultActiveInGame = (It.Value.DefaultCategoryState == EGameplayDebuggerCategoryState::EnabledInGame) || (It.Value.DefaultCategoryState == EGameplayDebuggerCategoryState::EnabledInGameAndSimulate);
				const bool bDefaultActiveInSimulate = (It.Value.DefaultCategoryState == EGameplayDebuggerCategoryState::EnabledInSimulate) || (It.Value.DefaultCategoryState == EGameplayDebuggerCategoryState::EnabledInGameAndSimulate);
				const bool bDefaultHidden = It.Value.DefaultCategoryState == EGameplayDebuggerCategoryState::Hidden;

				const bool bActiveInGame = (ConfigData.ActiveInGame == EGameplayDebuggerOverrideMode::UseDefault) ? bDefaultActiveInGame : (ConfigData.ActiveInGame == EGameplayDebuggerOverrideMode::Enable);
				const bool bActiveInSimulate = (ConfigData.ActiveInSimulate == EGameplayDebuggerOverrideMode::UseDefault) ? bDefaultActiveInSimulate : (ConfigData.ActiveInSimulate == EGameplayDebuggerOverrideMode::Enable);
				const bool bHidden = (ConfigData.Hidden == EGameplayDebuggerOverrideMode::UseDefault) ? bDefaultHidden : (ConfigData.Hidden == EGameplayDebuggerOverrideMode::Enable);

				EGameplayDebuggerCategoryState NewCategoryState =
					bHidden ? EGameplayDebuggerCategoryState::Hidden :
					bActiveInGame && bActiveInSimulate ? EGameplayDebuggerCategoryState::EnabledInGameAndSimulate :
					bActiveInGame ? EGameplayDebuggerCategoryState::EnabledInGame :
					bActiveInSimulate ? EGameplayDebuggerCategoryState::EnabledInSimulate :
					EGameplayDebuggerCategoryState::Disabled;

				bCategoriesChanged = bCategoriesChanged || (It.Value.SlotIdx != ConfigData.SlotIdx) || (It.Value.CategoryState != NewCategoryState);
				It.Value.SlotIdx = ConfigData.SlotIdx;
				It.Value.CategoryState = NewCategoryState;
				break;
			}
		}
	}

	bool bExtensionsChanged = false;
	for (auto& It : ExtensionMap)
	{
		for (int32 Idx = 0; Idx < ToolConfig->Extensions.Num(); Idx++)
		{
			const FGameplayDebuggerExtensionConfig& ConfigData = ToolConfig->Extensions[Idx];
			if (*ConfigData.ExtensionName == It.Key)
			{
				const bool bWantsEnabled = (ConfigData.UseExtension == EGameplayDebuggerOverrideMode::UseDefault) ? It.Value.bDefaultEnabled : (ConfigData.UseExtension == EGameplayDebuggerOverrideMode::Enable);
				bExtensionsChanged = bExtensionsChanged || (It.Value.bEnabled != bWantsEnabled);
				It.Value.bEnabled = bWantsEnabled;
				break;
			}
		}
	}

	if (bCategoriesChanged)
	{
		NotifyCategoriesChanged();
	}
	
	if (bExtensionsChanged)
	{
		NotifyExtensionsChanged();
	}
}
