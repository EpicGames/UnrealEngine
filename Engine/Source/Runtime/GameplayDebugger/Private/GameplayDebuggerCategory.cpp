// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayDebuggerCategory.h"
#include "Engine/World.h"
#include "GameFramework/PlayerController.h"
#include "GameplayDebuggerCategoryReplicator.h"
#include "GameplayDebuggerConfig.h"
#include "GameplayDebuggerPlayerManager.h"

FGameplayDebuggerCategory::FGameplayDebuggerCategory() :
	CollectDataInterval(0.0f),
	bShowDataPackReplication(false),
	bShowUpdateTimer(false),
	bShowCategoryName(true),
	bShowOnlyWithDebugActor(true),
	bAllowLocalDataCollection(false),
	bIsLocal(false),
	bHasAuthority(true),
	bIsEnabled(true),
	CategoryId(INDEX_NONE),
	LastCollectDataTime(-MAX_dbl)
{
}

FGameplayDebuggerCategory::~FGameplayDebuggerCategory()
{
}

void FGameplayDebuggerCategory::CollectData(APlayerController* OwnerPC, AActor* DebugActor)
{
	// empty in base class
}

void FGameplayDebuggerCategory::DrawData(APlayerController* OwnerPC, FGameplayDebuggerCanvasContext& CanvasContext)
{
	// empty in base class
}

FDebugRenderSceneProxy* FGameplayDebuggerCategory::CreateDebugSceneProxy(const UPrimitiveComponent* InComponent, FDebugDrawDelegateHelper*& OutDelegateHelper)
{
	OutDelegateHelper = nullptr;
	// empty in base class
	return nullptr;
}

void FGameplayDebuggerCategory::OnDataPackReplicated(int32 DataPackId)
{
	// empty in base class
}

void FGameplayDebuggerCategory::AddTextLine(const FString& TextLine)
{
	if (bHasAuthority || bAllowLocalDataCollection)
	{
		ReplicatedLines.Add(TextLine);
	}
}

void FGameplayDebuggerCategory::AddShape(const FGameplayDebuggerShape& Shape)
{
	if (bHasAuthority || bAllowLocalDataCollection)
	{
		ReplicatedShapes.Add(Shape);
	}
}

void FGameplayDebuggerCategory::DrawCategory(APlayerController* OwnerPC, FGameplayDebuggerCanvasContext& CanvasContext)
{
	// note that we prefer OwnerPC's world here since if it's given then that's the end user of the data
	UWorld* World = CanvasContext.GetWorld();

	if (!ensure(World))
	{
		return;
	}

	FString CategoryPrefix;
	if (!bShowCategoryName)
	{
		CategoryPrefix = FString::Printf(TEXT("{green}[%s]{white}  "), *CategoryName.ToString());
	}

	if (bShowUpdateTimer && bHasAuthority)
	{
		const double GameTime = World->GetTimeSeconds();
		CanvasContext.Printf(TEXT("%sNext update in: {yellow}%.0fs"), *CategoryPrefix, CollectDataInterval - (GameTime - LastCollectDataTime));
	}

	if (bShowDataPackReplication)
	{
		for (int32 Idx = 0; Idx < ReplicatedDataPacks.Num(); Idx++)
		{
			FGameplayDebuggerDataPack& DataPack = ReplicatedDataPacks[Idx];
			if (DataPack.IsInProgress())
			{
				const FString DataPackMessage = (ReplicatedDataPacks.Num() == 1) ?
					FString::Printf(TEXT("%sReplicating: {red}%.0f%% {white}(ver:%d)"), *CategoryPrefix, DataPack.GetProgress() * 100.0f, DataPack.Header.DataVersion) :
					FString::Printf(TEXT("%sReplicating data[%d]: {red}%.0f%% {white}(ver:%d)"), *CategoryPrefix, Idx, DataPack.GetProgress() * 100.0f, DataPack.Header.DataVersion);

				CanvasContext.Print(DataPackMessage);
			}
		}
	}

	for (int32 Idx = 0; Idx < ReplicatedLines.Num(); Idx++)
	{
		CanvasContext.Print(ReplicatedLines[Idx]);
	}

	for (int32 Idx = 0; Idx < ReplicatedShapes.Num(); Idx++)
	{
		ReplicatedShapes[Idx].Draw(World, CanvasContext);
	}

	DrawData(OwnerPC, CanvasContext);
}

bool FGameplayDebuggerCategory::GetViewPoint(const APlayerController* OwnerPC, FVector& OutViewLocation, FVector& OutViewDirection) const
{
	if (const AGameplayDebuggerCategoryReplicator* Replicator = GetReplicator())
	{
		if (Replicator->GetViewPoint(OutViewLocation, OutViewDirection))
		{
			return true;
		}
	}

	if (OwnerPC != nullptr)
	{
		AGameplayDebuggerPlayerManager::GetViewPoint(*OwnerPC, OutViewLocation, OutViewDirection);
		return true;
	}

	return false;
}

bool FGameplayDebuggerCategory::IsLocationInViewCone(const FVector& ViewLocation, const FVector& ViewDirection, const FVector& TargetLocation)
{
	const UGameplayDebuggerUserSettings* Settings = GetDefault<UGameplayDebuggerUserSettings>();
	const FVector DirToEntity = TargetLocation - ViewLocation;
	const FVector::FReal DistanceToEntitySq = DirToEntity.SquaredLength();
	if (DistanceToEntitySq > FMath::Square(Settings->MaxViewDistance))
	{
		return false;
	}

	const FVector::FReal ViewDot = FVector::DotProduct(DirToEntity.GetSafeNormal(), ViewDirection);
	const FVector::FReal MinViewDirDot = FMath::Cos(FMath::DegreesToRadians(Settings->MaxViewAngle));
	if (ViewDot < MinViewDirDot)
	{
		return false;
	}

	return true;
}

void FGameplayDebuggerCategory::MarkDataPackDirty(int32 DataPackId)
{
	if (ReplicatedDataPacks.IsValidIndex(DataPackId))
	{
		ReplicatedDataPacks[DataPackId].bIsDirty = true;
	}
}

void FGameplayDebuggerCategory::MarkRenderStateDirty()
{
	if (bIsLocal)
	{
		AGameplayDebuggerCategoryReplicator* RepOwnerOb = GetReplicator();
		if (RepOwnerOb)
		{
			RepOwnerOb->MarkComponentsRenderStateDirty();
		}
	}
}

FString FGameplayDebuggerCategory::GetSceneProxyViewFlag() const
{
	const bool bIsSimulate = FGameplayDebuggerAddonBase::IsSimulateInEditor();
	return bIsSimulate ? TEXT("DebugAI") : TEXT("Game");
}

void FGameplayDebuggerCategory::ResetReplicatedData()
{
	ReplicatedLines.Reset();
	ReplicatedShapes.Reset();
	ReplicatedDataPacks.Reset();
}
