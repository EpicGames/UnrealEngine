// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayDebuggerCategoryReplicator.h"
#include "Engine/World.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerController.h"
#include "GameplayDebuggerAddonBase.h"
#include "GameplayDebuggerAddonManager.h"
#include "GameplayDebuggerPlayerManager.h"
#include "GameplayDebuggerRenderingComponent.h"
#include "GameplayDebuggerExtension.h"
#include "Net/UnrealNetwork.h"
#include "Engine/NetDriver.h"
#include "Engine/NetConnection.h"
#include "VisualLogger/VisualLogger.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"

#include "Iris/IrisConfig.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Net/Iris/ReplicationSystem/ReplicationSystemUtil.h"
#include "Net/Iris/ReplicationSystem/EngineReplicationBridge.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayDebuggerCategoryReplicator)

#if WITH_EDITOR
#include "Editor.h"
#include "Engine/Selection.h"
#endif // WITH_EDITOR

//////////////////////////////////////////////////////////////////////////
// FGameplayDebuggerCategoryReplicatorData

DEFINE_LOG_CATEGORY_STATIC(LogGameplayDebugReplication, Display, All);

static TAutoConsoleVariable<int32> CVarGameplayDebuggerRepDetails(
	TEXT("ai.debug.DetailedReplicationLogs"),
	0,
	TEXT("Enable or disable very verbose replication logs for gameplay debugger"),
	ECVF_Cheat);

FNotifyGameplayDebuggerOwnerChange AGameplayDebuggerCategoryReplicator::NotifyDebuggerOwnerChange;

class FNetFastCategoryBaseState : public INetDeltaBaseState
{
public:
	struct FDataPackState
	{
		int32 DataOffset;
		int16 DataVersion;
		int16 SyncCounter;

		FDataPackState() : DataOffset(0), DataVersion(0), SyncCounter(0) {}
		FDataPackState(const FGameplayDebuggerDataPack::FHeader& Header) : DataOffset(Header.DataOffset), DataVersion(Header.DataVersion), SyncCounter(Header.SyncCounter) {}

		FORCEINLINE bool IsEqual(const FDataPackState& OtherState) const
		{
			return (DataOffset == OtherState.DataOffset) && (DataVersion == OtherState.DataVersion) && (SyncCounter == OtherState.SyncCounter);
		}

		FORCEINLINE bool operator==(const FDataPackState& OtherState) const { return IsEqual(OtherState); }
		FORCEINLINE bool operator!=(const FDataPackState& OtherState) const { return !IsEqual(OtherState); }
	};

	struct FCategoryState
	{
		int32 TextLinesRepCounter;
		int32 ShapesRepCounter;
		TArray<FDataPackState> DataPackStates;

		FCategoryState() : TextLinesRepCounter(0), ShapesRepCounter(0) {}

		FORCEINLINE bool IsEqual(const FCategoryState& OtherState) const
		{
			return (TextLinesRepCounter == OtherState.TextLinesRepCounter) &&
				(ShapesRepCounter == OtherState.ShapesRepCounter) &&
				(DataPackStates == OtherState.DataPackStates);
		}

		FORCEINLINE bool operator==(const FCategoryState& OtherState) const { return IsEqual(OtherState); }
		FORCEINLINE bool operator!=(const FCategoryState& OtherState) const { return !IsEqual(OtherState); }
	};

	virtual bool IsStateEqual(INetDeltaBaseState* OtherState) override
	{
		FNetFastCategoryBaseState* Other = static_cast<FNetFastCategoryBaseState*>(OtherState);
		return (CategoryStates == Other->CategoryStates);
	}

	void DumpToLog()
	{
		for (int32 CategoryIdx = 0; CategoryIdx < CategoryStates.Num(); CategoryIdx++)
		{
			const FCategoryState& CategoryData = CategoryStates[CategoryIdx];
			UE_LOGF(LogGameplayDebugReplication, Verbose, "category[%d] TextLinesRepCounter:%d ShapesRepCounter:%d",
				CategoryIdx, CategoryData.TextLinesRepCounter, CategoryData.ShapesRepCounter);

			for (int32 DataPackIdx = 0; DataPackIdx < CategoryData.DataPackStates.Num(); DataPackIdx++)
			{
				const FDataPackState& DataPack = CategoryData.DataPackStates[DataPackIdx];
				UE_LOGF(LogGameplayDebugReplication, Verbose, ">>    data[%d] DataVersion:%d SyncCounter:%d DataOffset:%d",
					DataPackIdx, DataPack.DataVersion, DataPack.SyncCounter, DataPack.DataOffset);
			}
		}
	}

	TArray<FCategoryState> CategoryStates;
};

void AGameplayDebuggerCategoryReplicator::PreReplication(IRepChangedPropertyTracker& ChangedPropertyTracker)
{
	if (UE::Net::FReplicationSystemUtil::GetNetHandle(this).IsValid())
	{
		ReplicatedData.PopulateFromOwner();
	}
}

void AGameplayDebuggerCategoryReplicator::OnRep_ReplicatedData()
{
	if (UE::Net::FReplicationSystemUtil::GetReplicationSystem(this))
	{
		ReplicatedData.ApplyToOwner();
	}
}

void FGameplayDebuggerNetPack::PopulateFromOwner()
{
	// The current implementation of the FGameplayDebuggerNetPack replication does not behave like most other 
	// replicated systems so in order to make this work for Iris replication which has much stricter rules for what can be done during serialization,
	// we cannot rely on polling being made during the call to the custom NetDeltaSerialize method so we use this explicit method to poll the data to be replicated instead.
	
	if (Owner && Owner->bIsEnabled && Owner->Categories.Num() == SavedData.Num())
	{
		// If receiving end are stopped in debugger etc. we should not post more RPCs.
		UNetConnection* Connection = Owner->GetNetConnection();
		if (!Connection || !Connection->IsNetReady() || Connection->IsPacketSequenceWindowFull(4))
		{
			return;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(FGameplayDebugger_PopulateFromOwner);

		// Update SavedCategories for replication, DataPacks-packets are handled using RPC`s
		for (int32 Idx = 0; Idx < SavedData.Num(); Idx++)
		{
			FGameplayDebuggerCategory& CategoryOb = Owner->Categories[Idx].Get();
			FGameplayDebuggerCategoryData& SavedCategory = SavedData[Idx];

			SavedCategory.CategoryName = CategoryOb.GetCategoryName();
			SavedCategory.bIsEnabled = CategoryOb.bIsEnabled;

			const bool bTextLinesChanged = (SavedCategory.TextLines != CategoryOb.ReplicatedLines);
			if (bTextLinesChanged)
			{
				SavedCategory.TextLines = CategoryOb.ReplicatedLines;
			}

			const bool bShapesChanged = (SavedCategory.Shapes != CategoryOb.ReplicatedShapes);
			if (bShapesChanged)
			{
				SavedCategory.Shapes = CategoryOb.ReplicatedShapes;
			}

			// Send the datapack, rpcs. One per category.
			if (Owner->bSendDataPacksUsingRPC)
			{
				// We use saveddata to track what we have sent
				SavedCategory.DataPacks.SetNum(CategoryOb.ReplicatedDataPacks.Num());

				for (int32 DataPackIdx = 0; DataPackIdx < CategoryOb.ReplicatedDataPacks.Num(); DataPackIdx++)
				{
					FGameplayDebuggerDataPack& DataPack = CategoryOb.ReplicatedDataPacks[DataPackIdx];
					if (!CategoryOb.bIsLocal)
					{
						if ((SavedCategory.DataPacks[DataPackIdx] != DataPack.Header) || (DataPack.bNeedsConfirmation && !DataPack.bReceived))
						{
							// Send the update data pack as an reliable rpc instead
							Owner->SendDataPackPacket(CategoryOb.GetCategoryName(), DataPackIdx, DataPack);						
							SavedCategory.DataPacks[DataPackIdx] = DataPack.Header;
						}
					}
				}
			}
		}
	}
}

void FGameplayDebuggerNetPack::ApplyToOwner()
{
	if (!ensure(Owner))
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FGameplayDebugger_ApplyToOwner);

	bool bHasCategoryStateChanges = false;
	for (const FGameplayDebuggerCategoryData& SavedDataEntry : SavedData)
	{
		// Does the category exist on the client?
		// using FindLastByPredicate since that's the only Predicate-based find that returns index.
		const FName CategoryName = SavedDataEntry.CategoryName;
		const int32 FoundIndex = Owner->Categories.FindLastByPredicate([CategoryName](const TSharedRef<FGameplayDebuggerCategory>& Item) { return (Item->GetCategoryName() == CategoryName); });
		if (FoundIndex == INDEX_NONE)
		{
			continue;
		}
		
		FGameplayDebuggerCategory& CategoryOb = Owner->Categories[FoundIndex].Get();
		UE_LOGF(LogGameplayDebugReplication, Verbose, "  CATEGORY[%d]:%ls", FoundIndex, *CategoryOb.GetCategoryName().ToString());
		
		if (CategoryOb.bIsEnabled != SavedDataEntry.bIsEnabled)
		{
			bHasCategoryStateChanges = true;
			CategoryOb.bIsEnabled = SavedDataEntry.bIsEnabled;
		}

		if (SavedDataEntry.TextLines != CategoryOb.ReplicatedLines)
		{

			UE_LOGF(LogGameplayDebugReplication, Verbose, "  >> received lines");
			CategoryOb.ReplicatedLines = SavedDataEntry.TextLines;
		}

		if (SavedDataEntry.Shapes != CategoryOb.ReplicatedShapes)
		{
			UE_LOGF(LogGameplayDebugReplication, Verbose, "  >> received shapes");
			CategoryOb.ReplicatedShapes = SavedDataEntry.Shapes;
		}
	}

	// force scene proxy updates if categories changed state
	if (bHasCategoryStateChanges)
	{
		Owner->MarkComponentsRenderStateDirty();
	}
}

bool FGameplayDebuggerNetPack::NetDeltaSerialize(FNetDeltaSerializeInfo& DeltaParms)
{
	if (DeltaParms.bUpdateUnmappedObjects || Owner == nullptr)
	{
		return true;
	}

	if (DeltaParms.Writer)
	{
		// watch for DemoNetDriver doing snapshot on client, no need to store debug data here
		// especially if it tries to write incomplete packets
		const bool bIsOwnerClient = !Owner->bHasAuthority;
		if (bIsOwnerClient)
		{
			return false;
		}

		FBitWriter& Writer = *DeltaParms.Writer;
		int32 NumChangedCategories = 0;

		FNetFastCategoryBaseState* OldState = static_cast<FNetFastCategoryBaseState*>(DeltaParms.OldState);
		FNetFastCategoryBaseState* NewState = nullptr;
		TArray<uint8> ChangedCategories;

		// find delta to replicate
		if (Owner->bIsEnabled && Owner->Categories.Num() == SavedData.Num())
		{
			NewState = new FNetFastCategoryBaseState();
			check(DeltaParms.NewState);
			*DeltaParms.NewState = TSharedPtr<INetDeltaBaseState>(NewState);
			NewState->CategoryStates.SetNum(SavedData.Num());
			ChangedCategories.AddZeroed(SavedData.Num());

			for (int32 Idx = 0; Idx < SavedData.Num(); Idx++)
			{
				FNetFastCategoryBaseState::FCategoryState& CategoryState = NewState->CategoryStates[Idx];
				FGameplayDebuggerCategory& CategoryOb = Owner->Categories[Idx].Get();
				FGameplayDebuggerCategoryData& SavedCategory = SavedData[Idx];

				const bool bMissingOldState = (OldState == nullptr) || !OldState->CategoryStates.IsValidIndex(Idx);
				ChangedCategories[Idx] = bMissingOldState ? 1 : 0;

				SavedCategory.CategoryName = CategoryOb.GetCategoryName();
				if (SavedCategory.bIsEnabled != CategoryOb.bIsEnabled)
				{
					SavedCategory.bIsEnabled = CategoryOb.bIsEnabled;
					ChangedCategories[Idx]++;
				}

				const bool bTextLinesChanged = (SavedCategory.TextLines != CategoryOb.ReplicatedLines);
				CategoryState.TextLinesRepCounter = (bMissingOldState ? 0 : OldState->CategoryStates[Idx].TextLinesRepCounter) + (bTextLinesChanged ? 1 : 0);
				if (bTextLinesChanged)
				{
					SavedCategory.TextLines = CategoryOb.ReplicatedLines;
					ChangedCategories[Idx]++;
				}

				const bool bShapesChanged = (SavedCategory.Shapes != CategoryOb.ReplicatedShapes);
				CategoryState.ShapesRepCounter = (bMissingOldState ? 0 : OldState->CategoryStates[Idx].ShapesRepCounter) + (bShapesChanged ? 1 : 0);
				if (bShapesChanged)
				{
					SavedCategory.Shapes = CategoryOb.ReplicatedShapes;
					ChangedCategories[Idx]++;
				}

				// If datapack-packets are sent using RPC`s we do not need to store them in the SavedCategoryStates or the Delta baselines
				const bool bUseDataPackRPC = Owner->bSendDataPacksUsingRPC;
				const int32 NumDataPacks = bUseDataPackRPC ? 0 : CategoryOb.ReplicatedDataPacks.Num();

				SavedCategory.DataPacks.SetNum(NumDataPacks);
				CategoryState.DataPackStates.SetNum(NumDataPacks);
				for (int32 DataIdx = 0; DataIdx < NumDataPacks; DataIdx++)
				{
					FGameplayDebuggerDataPack& DataPack = CategoryOb.ReplicatedDataPacks[DataIdx];
					const bool bHasOldStatePack = !bMissingOldState && OldState->CategoryStates[Idx].DataPackStates.IsValidIndex(DataIdx);

					if (DataPack.bNeedsConfirmation && !DataPack.bReceived && bHasOldStatePack)
					{
						FNetFastCategoryBaseState::FDataPackState& OldDataPackState = OldState->CategoryStates[Idx].DataPackStates[DataIdx];
						UE_LOGF(LogGameplayDebugReplication, Verbose, "Checking packet confirmation for Category[%d].DataPack[%d] OldState(DataVersion:%d DataOffset:%d complete:%ls) current(DataVersion:%d DataOffset:%d)",
							Idx, DataIdx,
							OldDataPackState.DataVersion, OldDataPackState.DataOffset,
							(OldDataPackState.DataOffset == DataPack.Header.DataSize) && (OldDataPackState.DataVersion == DataPack.Header.DataVersion) ? TEXT("yes") : TEXT("no"),
							DataPack.Header.DataVersion, DataPack.Header.DataOffset);

						DataPack.OnPacketRequest(OldDataPackState.DataVersion, OldDataPackState.DataOffset);
					}

					CategoryState.DataPackStates[DataIdx] = FNetFastCategoryBaseState::FDataPackState(DataPack.Header);
					const bool bDataPackChanged = (SavedCategory.DataPacks[DataIdx] != DataPack.Header);
					if (bDataPackChanged)
					{
						SavedCategory.DataPacks[DataIdx] = DataPack.Header;
						ChangedCategories[Idx]++;
					}
					else if (bHasOldStatePack)
					{
						FNetFastCategoryBaseState::FDataPackState& OldDataPackState = OldState->CategoryStates[Idx].DataPackStates[DataIdx];
						const bool bDataPackNotUpdatedOnClient = (OldDataPackState != DataPack.Header);
						if (bDataPackNotUpdatedOnClient)
						{
							ChangedCategories[Idx]++;
						}
					}
				}

				NumChangedCategories += ChangedCategories[Idx] ? 1 : 0;
			}
		}

		if (CVarGameplayDebuggerRepDetails.GetValueOnAnyThread())
		{
			if (OldState)
			{
				UE_LOGF(LogGameplayDebugReplication, Verbose, "NetDeltaSerialize DUMP OldState");
				OldState->DumpToLog();
			}
			if (NewState)
			{
				UE_LOGF(LogGameplayDebugReplication, Verbose, "NetDeltaSerialize DUMP NewState");
				NewState->DumpToLog();
			}
		}

		if (NumChangedCategories == 0)
		{
			return false;
		}

		int32 CategoryCount = SavedData.Num();
		Writer << CategoryCount;

		UE_LOGF(LogGameplayDebugReplication, Verbose, "NetDeltaSerialize WRITE START, OldState:%d", OldState ? 1 : 0);
		for (int32 Idx = 0; Idx < SavedData.Num(); Idx++)
		{
			FGameplayDebuggerCategory& CategoryOb = Owner->Categories[Idx].Get();
			const bool bMissingOldState = (OldState == nullptr) || !OldState->CategoryStates.IsValidIndex(Idx);
			FGameplayDebuggerCategoryData& SavedCategory = SavedData[Idx];

			uint8 BaseFlags = SavedCategory.bIsEnabled;
			uint8 ShouldUpdateTextLines = bMissingOldState || (OldState->CategoryStates[Idx].TextLinesRepCounter != NewState->CategoryStates[Idx].TextLinesRepCounter);
			uint8 ShouldUpdateShapes = bMissingOldState || (OldState->CategoryStates[Idx].ShapesRepCounter != NewState->CategoryStates[Idx].ShapesRepCounter);
			uint8 NumDataPacks = IntCastChecked<uint8>(SavedCategory.DataPacks.Num());

			Writer << SavedCategory.CategoryName;

			Writer.WriteBit(BaseFlags);
			Writer.WriteBit(ShouldUpdateTextLines);
			Writer.WriteBit(ShouldUpdateShapes);
			Writer << NumDataPacks;

			if (ChangedCategories[Idx])
			{
				UE_LOGF(LogGameplayDebugReplication, Verbose, "  CATEGORY[%d]:%ls", Idx, *CategoryOb.GetCategoryName().ToString());
			}

			if (ShouldUpdateTextLines)
			{
				UE_LOGF(LogGameplayDebugReplication, Verbose, "  >> replicate lines");
				Writer << SavedCategory.TextLines;
			}

			if (ShouldUpdateShapes)
			{
				UE_LOGF(LogGameplayDebugReplication, Verbose, "  >> replicate shapes");
				Writer << SavedCategory.Shapes;
			}

			for (int32 DataIdx = 0; DataIdx < NumDataPacks; DataIdx++)
			{
				uint8 ShouldUpdateDataPack = bMissingOldState || !OldState->CategoryStates[Idx].DataPackStates.IsValidIndex(DataIdx) ||
					(OldState->CategoryStates[Idx].DataPackStates[DataIdx] != NewState->CategoryStates[Idx].DataPackStates[DataIdx]);

				Writer.WriteBit(ShouldUpdateDataPack);
				if (ShouldUpdateDataPack)
				{
					// send single packet
					FGameplayDebuggerDataPack& DataPack = CategoryOb.ReplicatedDataPacks[DataIdx];

					uint8 IsCompressed = DataPack.Header.bIsCompressed ? 1 : 0;
					Writer.WriteBit(IsCompressed);

					Writer << DataPack.Header.DataVersion;
					Writer << DataPack.Header.SyncCounter;
					Writer << DataPack.Header.DataSize;
					Writer << DataPack.Header.DataOffset;

					const int32 PacketSize = FMath::Min(FGameplayDebuggerDataPack::PacketSize, DataPack.Header.DataSize - DataPack.Header.DataOffset);
					if (PacketSize > 0)
					{
						Writer.Serialize(DataPack.Data.GetData() + DataPack.Header.DataOffset, PacketSize);
					}

					UE_LOGF(LogGameplayDebugReplication, Verbose, "  >> replicate data pack[%d] progress:%0.f%% (offset:%d packet:%d)",
						DataIdx, DataPack.Header.DataSize ? (100.0f * (DataPack.Header.DataOffset + PacketSize) / DataPack.Header.DataSize) : 100.0f,
						DataPack.Header.DataOffset, PacketSize);
				}
			}
		}
	}
	else if (DeltaParms.Reader)
	{
		FBitReader& Reader = *DeltaParms.Reader;
		UE_LOGF(LogGameplayDebugReplication, Verbose, "NetDeltaSerialize READ START");

		int32 CategoryCount = 0;
		Reader << CategoryCount;

		// Replicated categories still have to be serialized even if they don't exist on the client.
		// Create a dummy category for these cases
		FGameplayDebuggerCategory DummyCategory;

		bool bHasCategoryStateChanges = false;
		for (int32 Idx = 0; Idx < CategoryCount; Idx++)
		{
			FName CategoryName;
			Reader << CategoryName;

			// Does the category exist on the client?
			// using FindLastByPredicate since that's the only Predicate-based find that returns index.
			const int32 FoundIndex = Owner->Categories.FindLastByPredicate([CategoryName](const TSharedRef<FGameplayDebuggerCategory>& Item)
				{
					return (Item->GetCategoryName() == CategoryName);
				});

			FGameplayDebuggerCategory& CategoryOb = (Owner->Categories.IsValidIndex(FoundIndex)) ? Owner->Categories[FoundIndex].Get() : DummyCategory;
			UE_LOGF(LogGameplayDebugReplication, Verbose, "  CATEGORY[%d]:%ls", FoundIndex, *CategoryOb.GetCategoryName().ToString());

			uint8 BaseFlags = Reader.ReadBit();
			uint8 ShouldUpdateTextLines = Reader.ReadBit();
			uint8 ShouldUpdateShapes = Reader.ReadBit();

			uint8 NumDataPacks = 0;
			Reader << NumDataPacks;

			if (FoundIndex != INDEX_NONE && (int32)NumDataPacks != CategoryOb.ReplicatedDataPacks.Num())
			{
				UE_LOGF(LogGameplayDebugReplication, Error, "Data pack count mismatch! received:%d expected:%d", NumDataPacks, CategoryOb.ReplicatedDataPacks.Num());
				Reader.SetError();
				return false;
			}

			const bool bNewCategoryEnabled = (BaseFlags != 0);
			bHasCategoryStateChanges = bHasCategoryStateChanges || (CategoryOb.bIsEnabled != bNewCategoryEnabled);
			CategoryOb.bIsEnabled = bNewCategoryEnabled;

			if (ShouldUpdateTextLines)
			{
				UE_LOGF(LogGameplayDebugReplication, Verbose, "  >> received lines");
				Reader << CategoryOb.ReplicatedLines;
			}

			if (ShouldUpdateShapes)
			{
				UE_LOGF(LogGameplayDebugReplication, Verbose, "  >> received shapes");
				Reader << CategoryOb.ReplicatedShapes;
			}

			for (uint8 DataIdx = 0; DataIdx < NumDataPacks; DataIdx++)
			{
				uint8 ShouldUpdateDataPack = Reader.ReadBit();

				if (ShouldUpdateDataPack)
				{
					// receive single packet
					FGameplayDebuggerDataPack DataPacket;

					uint8 IsCompressed = Reader.ReadBit();
					DataPacket.Header.bIsCompressed = (IsCompressed != 0);

					Reader << DataPacket.Header.DataVersion;
					Reader << DataPacket.Header.SyncCounter;
					Reader << DataPacket.Header.DataSize;
					Reader << DataPacket.Header.DataOffset;

					const int32 PacketSize = FMath::Min(FGameplayDebuggerDataPack::PacketSize, DataPacket.Header.DataSize - DataPacket.Header.DataOffset);
					if (PacketSize > 0)
					{
						DataPacket.Data.AddUninitialized(PacketSize);
						Reader.Serialize(DataPacket.Data.GetData(), PacketSize);
					}

					Owner->OnReceivedDataPackPacket(FoundIndex, DataIdx, DataPacket);
					UE_LOGF(LogGameplayDebugReplication, Verbose, "  >> replicate data pack[%d] progress:%.0f%%", DataIdx, CategoryOb.ReplicatedDataPacks[DataIdx].GetProgress() * 100.0f);
				}
			}
		}

		// force scene proxy updates if categories changed state
		if (bHasCategoryStateChanges)
		{
			Owner->MarkComponentsRenderStateDirty();
		}
	}

	return true;
}

void FGameplayDebuggerNetPack::OnCategoriesChanged()
{
	SavedData.Reset();
	SavedData.SetNum(Owner->Categories.Num());
}

//////////////////////////////////////////////////////////////////////////
// AGameplayDebuggerCategoryReplicator

AGameplayDebuggerCategoryReplicator::AGameplayDebuggerCategoryReplicator(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.bAllowTickOnDedicatedServer = true;
	PrimaryActorTick.bTickEvenWhenPaused = true;
	PrimaryActorTick.bStartWithTickEnabled = false;

#if WITH_EDITOR
	SetIsTemporarilyHiddenInEditor(true);
#endif

#if WITH_EDITORONLY_DATA
	bHiddenEdLevel = true;
	bHiddenEdLayer = true;
	bHiddenEd = true;
	bEditable = false;
#endif

	bIsEnabled = false;
	bIsEnabledLocal = false;
	bHasAuthority = false;
	bIsLocal = false;
	bIsEditorWorldReplicator = false;
	bReplicates = true;

	ReplicatedData.Owner = this;

	bSendDataPacksUsingRPC = false;
	bOnlyRelevantToOwner = true;
}

void AGameplayDebuggerCategoryReplicator::BeginPlay()
{
	Super::BeginPlay();

	UWorld* World = GetWorld();
	check(World);
	const ENetMode NetMode = World->GetNetMode();
	bHasAuthority = FGameplayDebuggerUtils::IsAuthority(World);
	bIsLocal = !OwnerPC || OwnerPC->IsLocalPlayerController();

	Init();
}

void AGameplayDebuggerCategoryReplicator::OnReplicationStartedForIris(const FOnReplicationStartedParams& Params)
{
	if (UReplicationSystem* ReplicationSystem = UE::Net::FReplicationSystemUtil::GetReplicationSystem(this))
	{
		// If iris is enabled, datapack-packets are always sent using RPC`s and we also requires PreReplication to called in order to populate replicated data outside of serialization.
		SetCallPreReplication(true);
		bSendDataPacksUsingRPC = true;

		// Mark this as a debugobject to allow it to write more data not counting towards the bandwidth limits
		if (UEngineReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridgeAs<UEngineReplicationBridge>())
		{
			ReplicationSystem->SetIsDebugObject(Bridge->GetReplicatedRefHandle(this));
			ReplicationSystem->SetStaticPriority(Bridge->GetReplicatedRefHandle(this), 10.f);
		}
	}

	Super::OnReplicationStartedForIris(Params);
}

const AActor* AGameplayDebuggerCategoryReplicator::GetNetOwner() const 
{
	return OwnerPC.Get();
}

UPlayer* AGameplayDebuggerCategoryReplicator::GetNetOwningPlayer()
{
	if (const AActor* NetOwner = GetNetOwner())
	{
		if (NetOwner->GetLocalRole() == ROLE_Authority)
		{
			return GetNetOwningPlayerAnyRole();
		}
	}

	return nullptr;
}

UPlayer* AGameplayDebuggerCategoryReplicator::GetNetOwningPlayerAnyRole()
{
	if (AActor* NetOwner = OwnerPC.Get())
	{
		return NetOwner->GetNetOwningPlayerAnyRole();
	}

	return nullptr;
}

#if WITH_EDITOR
void AGameplayDebuggerCategoryReplicator::InitForEditor()
{
	bHasAuthority = true;
	bIsLocal = true;
	bIsEditorWorldReplicator = true;
	Init();
}
#endif // WITH_EDITOR

void AGameplayDebuggerCategoryReplicator::Init()
{
	UWorld* World = GetWorld();
	check(World);

	FGameplayDebuggerAddonManager& AddonManager = FGameplayDebuggerAddonManager::GetCurrent();
	AddonManager.OnCategoriesChanged.AddUObject(this, &AGameplayDebuggerCategoryReplicator::OnCategoriesChanged);
	AddonManager.OnExtensionsChanged.AddUObject(this, &AGameplayDebuggerCategoryReplicator::OnExtensionsChanged);

	OnCategoriesChanged();
	OnExtensionsChanged();

	SetActorHiddenInGame(!bIsLocal);
	if (bIsLocal)
	{
		RenderingComp = NewObject<UGameplayDebuggerRenderingComponent>(this, TEXT("RenderingComp"));
		RenderingComp->RegisterComponentWithWorld(World);
		RootComponent = RenderingComp;
	}
	
	if (bHasAuthority)
	{
		SetEnabled(FGameplayDebuggerAddonBase::IsSimulateInEditor());
	}

	AGameplayDebuggerPlayerManager& PlayerManager = AGameplayDebuggerPlayerManager::GetCurrent(World);
	PlayerManager.RegisterReplicator(*this);
}

void AGameplayDebuggerCategoryReplicator::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	Super::EndPlay(EndPlayReason);

	SetReplicatorOwner(nullptr);

	// Disable extensions to clear UI state
	NotifyCategoriesToolState(false);
	NotifyExtensionsToolState(false);

	FGameplayDebuggerAddonManager& AddonManager = FGameplayDebuggerAddonManager::GetCurrent();
	AddonManager.OnCategoriesChanged.RemoveAll(this);
	AddonManager.OnExtensionsChanged.RemoveAll(this);
}

void AGameplayDebuggerCategoryReplicator::OnCategoriesChanged()
{
	FGameplayDebuggerAddonManager& AddonManager = FGameplayDebuggerAddonManager::GetCurrent();
	AddonManager.CreateCategories(*this, Categories);

	ReplicatedData.OnCategoriesChanged();

	if (bIsLocal)
	{
		AGameplayDebuggerPlayerManager& PlayerManager = AGameplayDebuggerPlayerManager::GetCurrent(GetWorld());
		PlayerManager.RefreshInputBindings(*this);
	}
}

void AGameplayDebuggerCategoryReplicator::OnExtensionsChanged()
{
	FGameplayDebuggerAddonManager& AddonManager = FGameplayDebuggerAddonManager::GetCurrent();
	AddonManager.CreateExtensions(*this, Extensions);

	if (bIsLocal)
	{
		AGameplayDebuggerPlayerManager& PlayerManager = AGameplayDebuggerPlayerManager::GetCurrent(GetWorld());
		PlayerManager.RefreshInputBindings(*this);
	}
}

UNetConnection* AGameplayDebuggerCategoryReplicator::GetNetConnection() const
{
	return (OwnerPC.GetResidence() != EResidence::Remote && IsValid(OwnerPC)) ? OwnerPC->GetNetConnection() : nullptr;
}

bool AGameplayDebuggerCategoryReplicator::IsNetRelevantFor(const AActor* RealViewer, const AActor* ViewTarget, const FVector& SrcLocation) const
{
	return (RealViewer == OwnerPC);
}

/// @cond DOXYGEN_WARNINGS

void AGameplayDebuggerCategoryReplicator::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	DOREPLIFETIME(AGameplayDebuggerCategoryReplicator, OwnerPC);
	DOREPLIFETIME(AGameplayDebuggerCategoryReplicator, DebugActor);
	DOREPLIFETIME(AGameplayDebuggerCategoryReplicator, VisLogSync);
	DOREPLIFETIME(AGameplayDebuggerCategoryReplicator, bIsEnabled);
	DOREPLIFETIME(AGameplayDebuggerCategoryReplicator, ReplicatedData);

	// The visibility of the replicator actor is code driven (hidden on dedicated server, visible otherwise) and should not be replicated
	DISABLE_REPLICATED_PRIVATE_PROPERTY(AActor, bHidden);
}

bool AGameplayDebuggerCategoryReplicator::ServerSetEnabled_Validate(bool bEnable)
{
	return true;
}

void AGameplayDebuggerCategoryReplicator::ServerSetEnabled_Implementation(bool bEnable)
{
	SetEnabled(bEnable);
}

bool AGameplayDebuggerCategoryReplicator::ServerSetDebugActor_Validate(AActor* Actor, bool bSelectInEditor)
{
	return true;
}

void AGameplayDebuggerCategoryReplicator::ServerSetDebugActor_Implementation(AActor* Actor, bool bSelectInEditor)
{
	SetDebugActor(Actor, bSelectInEditor);
}

bool AGameplayDebuggerCategoryReplicator::ServerSetViewPoint_Validate(const FVector& InViewLocation, const FVector& InViewDirection)
{
	return true;
}

void AGameplayDebuggerCategoryReplicator::ServerSetViewPoint_Implementation(const FVector& InViewLocation, const FVector& InViewDirection)
{
	SetViewPoint(InViewLocation, InViewDirection);
}

bool AGameplayDebuggerCategoryReplicator::ServerResetViewPoint_Validate()
{
	return true;
}

void AGameplayDebuggerCategoryReplicator::ServerResetViewPoint_Implementation()
{
	ResetViewPoint();
}

bool AGameplayDebuggerCategoryReplicator::ServerSetCategoryEnabled_Validate(int32 CategoryId, bool bEnable)
{
	return true;
}

void AGameplayDebuggerCategoryReplicator::ServerSetCategoryEnabled_Implementation(int32 CategoryId, bool bEnable)
{
	UE_LOGF(LogGameplayDebugReplication, Error, "%s is Deprecated. Use ServerSetCategoryNameEnabled", __func__);
	SetCategoryEnabled(CategoryId, bEnable);
}

bool AGameplayDebuggerCategoryReplicator::ServerSetCategoryNameEnabled_Validate(FName CategoryName, bool bEnable)
{
	return true;
}

void AGameplayDebuggerCategoryReplicator::ServerSetCategoryNameEnabled_Implementation(FName CategoryName, bool bEnable)
{
	const int32 CategoryIndex = Categories.IndexOfByPredicate([CategoryName](const TSharedRef<FGameplayDebuggerCategory>& Item)
	{
		return Item->GetCategoryName() == CategoryName;
	});

	if (CategoryIndex != INDEX_NONE)
	{
		SetCategoryEnabled(CategoryIndex, bEnable);
	}
}

bool AGameplayDebuggerCategoryReplicator::ServerSendCategoryInputEvent_Validate(int32 CategoryId, int32 HandlerId)
{
	return true;
}

void AGameplayDebuggerCategoryReplicator::ServerSendCategoryInputEvent_Implementation(int32 CategoryId, int32 HandlerId)
{
	UE_LOGF(LogGameplayDebugReplication, Error, "%s is Deprecated. Use ServerSendCategoryNameInputEvent", __func__);
	SendCategoryInputEvent(CategoryId, HandlerId);
}

bool AGameplayDebuggerCategoryReplicator::ServerSendCategoryNameInputEvent_Validate(FName CategoryName, int32 HandlerId)
{
	return true;
}

void AGameplayDebuggerCategoryReplicator::ServerSendCategoryNameInputEvent_Implementation(FName CategoryName, int32 HandlerId)
{
	const int32 CategoryIndex = Categories.IndexOfByPredicate([CategoryName](const TSharedRef<FGameplayDebuggerCategory>& Item)
	{
		return Item->GetCategoryName() == CategoryName;
	});

	if (CategoryIndex != INDEX_NONE)
	{
		SendCategoryInputEvent(CategoryIndex, HandlerId);
	}	
}

bool AGameplayDebuggerCategoryReplicator::ServerSendExtensionInputEvent_Validate(int32 ExtensionId, int32 HandlerId)
{
	return true;
}

void AGameplayDebuggerCategoryReplicator::ServerSendExtensionInputEvent_Implementation(int32 ExtensionId, int32 HandlerId)
{
	SendExtensionInputEvent(ExtensionId, HandlerId);
}

/// @endcond

void AGameplayDebuggerCategoryReplicator::OnReceivedDataPackPacket(int32 CategoryId, int32 DataPackId, const FGameplayDebuggerDataPack& DataPacket)
{
	if (Categories.IsValidIndex(CategoryId) && Categories[CategoryId]->ReplicatedDataPacks.IsValidIndex(DataPackId))
	{
		FGameplayDebuggerDataPack& DataPack = Categories[CategoryId]->ReplicatedDataPacks[DataPackId];
		bool bIsPacketValid = false;

		if (DataPack.Header.DataVersion != DataPacket.Header.DataVersion)
		{
			// new content of data pack:
			if (DataPacket.Header.DataOffset == 0)
			{
				// first packet of data, replace old data pack's intermediate data
				DataPack.Header = DataPacket.Header;
				DataPack.Data = DataPacket.Data;
				bIsPacketValid = true;
			}
			else
			{
				// somewhere in the middle, discard packet
				UE_LOGF(LogGameplayDebugReplication, Verbose, "Error: received packet from the middle of content with different version, discarding! (category[%d]:%ls, packet.DataVersion:%d packet.DataOffset:%d, data[%d].DataVersion:%d)",
					CategoryId, *Categories[CategoryId]->GetCategoryName().ToString(),
					DataPacket.Header.DataVersion,
					DataPacket.Header.DataOffset,
					DataPackId, DataPack.Header.DataVersion);
			}
		}
		else if (DataPack.Data.Num() < DataPacket.Header.DataSize)
		{
			// another packet for existing data pack
			if (DataPacket.Header.DataOffset == DataPack.Data.Num())
			{
				// offset matches, this is next expected packet
				DataPack.Data.Append(DataPacket.Data);
				DataPack.Header.DataOffset = DataPack.Data.Num();
				bIsPacketValid = true;
			}
			else
			{
				// offset mismatch, discard packet
				UE_LOGF(LogGameplayDebugReplication, Verbose, "Error: received packet doesn't match expected chunk, discarding! (category[%d]:%ls, packet.DataOffset:%d, data[%d].DataOffset:%d data[%d].Data.Num:%d)",
					CategoryId, *Categories[CategoryId]->GetCategoryName().ToString(),
					DataPacket.Header.DataOffset,
					DataPackId, DataPack.Header.DataOffset,
					DataPackId, DataPack.Data.Num());
			}
		}

		// check if data pack is now complete
		if (bIsPacketValid && (DataPack.Data.Num() == DataPack.Header.DataSize))
		{
			// complete
			UE_LOGF(LogGameplayDebugReplication, Verbose, "Category[%d].DataPack[%d] RECEIVED, DataVersion:%d DataSize:%d SyncCounter:%d",
				CategoryId, DataPackId, DataPack.Header.DataVersion, DataPack.Header.DataSize, DataPack.Header.SyncCounter);

			DataPack.OnReplicated();
			Categories[CategoryId]->OnDataPackReplicated(DataPackId);
		}
	}
}

void AGameplayDebuggerCategoryReplicator::TickActor(float DeltaTime, enum ELevelTick TickType, FActorTickFunction& ThisTickFunction)
{
	Super::TickActor(DeltaTime, TickType, ThisTickFunction);
	if (OwnerPC || bIsEditorWorldReplicator)
	{
		CollectCategoryData();
	}
}

void AGameplayDebuggerCategoryReplicator::PostNetReceive()
{
	Super::PostNetReceive();

	// force scene proxy updates if tool changed state
	if (bIsEnabled != bIsEnabledLocal)
	{
		bIsEnabledLocal = bIsEnabled;
		MarkComponentsRenderStateDirty();
	}
}

void AGameplayDebuggerCategoryReplicator::CollectCategoryData(bool bForce)
{
	const double GameTime = GetWorld()->GetTimeSeconds();

#if ENABLE_VISUAL_LOG
	const FVisualLogger& VLogger = FVisualLogger::Get();
	VisLogSync.DeviceIDs.Reset();
	if (VLogger.IsRecordingToFile())
	{
		const TArray<FVisualLogDevice*>& LogDevices = VLogger.GetDevices();
		for (const FVisualLogDevice* Device : LogDevices)
		{
			VisLogSync.DeviceIDs += FString::Printf(TEXT("%u, "), Device ? Device->GetShortSessionID() : 0);
		}
	}
#endif // ENABLE_VISUAL_LOG

	for (int32 Idx = 0; Idx < Categories.Num(); Idx++)
	{
		FGameplayDebuggerCategory& CategoryOb = Categories[Idx].Get();
		if (CategoryOb.bHasAuthority && CategoryOb.bIsEnabled && (bForce || (GameTime - CategoryOb.LastCollectDataTime) > CategoryOb.CollectDataInterval))
		{
			// prepare data packs before calling CollectData
			for (int32 DataPackIdx = 0; DataPackIdx < CategoryOb.ReplicatedDataPacks.Num(); DataPackIdx++)
			{
				FGameplayDebuggerDataPack& DataPack = CategoryOb.ReplicatedDataPacks[DataPackIdx];
				DataPack.bIsDirty = false;

				if ((DataPack.Flags == EGameplayDebuggerDataPack::ResetOnTick) ||
					(DataPack.Flags == EGameplayDebuggerDataPack::ResetOnActorChange && DataPack.Header.SyncCounter != DebugActor.SyncCounter))
				{
					DataPack.ResetDelegate.Execute();
				}
			}

			CategoryOb.ReplicatedLines.Reset();
			CategoryOb.ReplicatedShapes.Reset();

			CategoryOb.CollectData(OwnerPC, DebugActor.Actor.Get());
			CategoryOb.LastCollectDataTime = GameTime;

			// update dirty data packs
			for (int32 DataPackIdx = 0; DataPackIdx < CategoryOb.ReplicatedDataPacks.Num(); DataPackIdx++)
			{
				FGameplayDebuggerDataPack& DataPack = CategoryOb.ReplicatedDataPacks[DataPackIdx];
				if (CategoryOb.bIsLocal)
				{
					const bool bWasDirty = DataPack.CheckDirtyAndUpdate();
					if (bWasDirty)
					{
						CategoryOb.OnDataPackReplicated(DataPackIdx);
					}

					if (CategoryOb.bHasAuthority)
					{
						// update sync counter for local & auth packs (no data replication), otherwise they can be reset
						DataPack.Header.SyncCounter = DebugActor.SyncCounter;
					}
				}
				else
				{
					const bool bWasDirty = DataPack.RequestReplication(DebugActor.SyncCounter);
					if (bWasDirty)
					{
						UE_LOGF(LogGameplayDebugReplication, Verbose, "Category[%d].DataPack[%d] SENT, DataVersion:%d DataSize:%d SyncCounter:%d",
							Idx, DataPackIdx, DataPack.Header.DataVersion, DataPack.Header.DataSize, DataPack.Header.SyncCounter);
					}
				}
			}
		}
	}
}

void AGameplayDebuggerCategoryReplicator::SendDataPackPacket(FName CategoryName, int32 DataPackIdx, FGameplayDebuggerDataPack& DataPack)
{
	FGameplayDebuggerDataPackRPCParams Params;

	Params.CategoryName = CategoryName;
	Params.DataPackIdx = DataPackIdx;
	Params.Header = DataPack.Header;

	const int32 PacketSize = FMath::Min(FGameplayDebuggerDataPack::PacketSize, DataPack.Header.DataSize - DataPack.Header.DataOffset);
	if (PacketSize > 0)
	{
		Params.Data.Append(DataPack.Data.GetData() + DataPack.Header.DataOffset, PacketSize);
	}

	UE_LOGF(LogGameplayDebugReplication, Verbose, "  >> replicate data pack[%d] RPC progress:%0.f%% (offset:%d packet:%d)",
		DataPackIdx, DataPack.Header.DataSize ? (100.0f * (DataPack.Header.DataOffset + PacketSize) / DataPack.Header.DataSize) : 100.0f,
		DataPack.Header.DataOffset, PacketSize);

	ClientDataPackPacket(Params);

	// As we are sending this as a reliable RPC we "ack" it immediately
	if (DataPack.bNeedsConfirmation && !DataPack.bReceived)
	{
		DataPack.OnPacketRequest(Params.Header.DataVersion, Params.Header.DataOffset);
	}
};

void AGameplayDebuggerCategoryReplicator::ClientDataPackPacket_Implementation(const FGameplayDebuggerDataPackRPCParams& Params)
{
	// Does the category exist on the client?
	// using FindLastByPredicate since that's the only Predicate-based find that returns index.
	const FName CategoryName = Params.CategoryName;
	const int32 FoundIndex = Categories.FindLastByPredicate([CategoryName](const TSharedRef<FGameplayDebuggerCategory>& Item) { return (Item->GetCategoryName() == CategoryName);});	
	if (FoundIndex == INDEX_NONE)
	{
		return;
	}

	// receive single packet
	FGameplayDebuggerDataPack DataPacket;

	DataPacket.Header = Params.Header;

	const int32 PacketSize = FMath::Min(FGameplayDebuggerDataPack::PacketSize, DataPacket.Header.DataSize - DataPacket.Header.DataOffset);
	if (PacketSize > 0)
	{
		DataPacket.Data.Append(Params.Data.GetData(), PacketSize);
	}
	
	OnReceivedDataPackPacket(FoundIndex, Params.DataPackIdx, DataPacket);

	const FGameplayDebuggerCategory& CategoryOb = Categories[FoundIndex].Get();
	UE_LOGF(LogGameplayDebugReplication, Verbose, "  >> replicate data pack[%d] progress:%.0f%%", Params.DataPackIdx, CategoryOb.ReplicatedDataPacks[Params.DataPackIdx].GetProgress() * 100.0f);	
}

bool AGameplayDebuggerCategoryReplicator::GetViewPoint(FVector& OutViewLocation, FVector& OutViewDirection) const
{
	if (IsViewPointSet())
	{
		OutViewLocation = ViewLocation.GetValue();
		OutViewDirection = ViewDirection.GetValue();
		return true;
	}

	return false;
}

void AGameplayDebuggerCategoryReplicator::SetReplicatorOwner(APlayerController* InOwnerPC)
{
	if (!bIsEnabled)
	{
		// can't use bHasAuthority, BeginPlay was not called yet
		UWorld* World = GetWorld();
		if (FGameplayDebuggerUtils::IsAuthority(World))
		{
			APlayerController* OldOwner = OwnerPC;
			OwnerPC = InOwnerPC;
			NotifyDebuggerOwnerChange.Broadcast(this, OldOwner);

			UE_VLOG_UELOG(this, LogGameplayDebugReplication, Log, TEXT("Set OWNER PC %s / %s / %s")
				, *GetNameSafe(InOwnerPC), InOwnerPC ? *GetNameSafe(InOwnerPC->GetPawn()) : TEXT_EMPTY
				, InOwnerPC ? *InOwnerPC->GetHumanReadableName() : TEXT_EMPTY);
		}
	}
}

void AGameplayDebuggerCategoryReplicator::SetEnabled(bool bEnable)
{
	UE_VLOG_UELOG(this, LogGameplayDebugReplication, Log, TEXT("SetEnabled %s"), TEXT_CONDITION(bEnable));

	if (bHasAuthority)
	{
		bIsEnabled = bEnable;
		bIsEnabledLocal = bEnable;
		SetActorTickEnabled(bEnable);
	}
	else
	{
		ServerSetEnabled(bEnable);
	}

	MarkComponentsRenderStateDirty();
	NotifyCategoriesToolState(bEnable);

	// extensions will NOT work with simulate mode, they are meant to handle additional input
	const bool bEnableExtensions = bEnable && !FGameplayDebuggerAddonBase::IsSimulateInEditor();
	NotifyExtensionsToolState(bEnableExtensions);
}

void AGameplayDebuggerCategoryReplicator::SetDebugActor(AActor* Actor, bool bSelectInEditor)
{
	UE_VLOG_UELOG(this, LogGameplayDebugReplication, Log, TEXT("SetDebugActor %s"), *GetNameSafe(Actor));	
	if (bHasAuthority)
	{
		if (DebugActor.Actor != Actor)
		{
			DebugActor.Actor = Actor;
			DebugActor.ActorName = Actor ? Actor->GetFName() : NAME_None;
			DebugActor.SyncCounter++;

#if WITH_EDITOR
			if (bSelectInEditor)
			{
				USelection* SelectedActors = GEditor ? GEditor->GetSelectedActors() : NULL;
				if (SelectedActors && Actor)
				{
					SelectedActors->DeselectAll();
					SelectedActors->Select(Actor);
				}
			}
#endif // WITH_EDITOR
		}
	}
	else
	{
		ServerSetDebugActor(Actor, bSelectInEditor);
	}
}

void AGameplayDebuggerCategoryReplicator::SetViewPoint(const FVector& InViewLocation, const FVector& InViewDirection)
{
	ViewLocation = InViewLocation;
	ViewDirection = InViewDirection;

	if (!bHasAuthority)
	{
		ServerSetViewPoint(InViewLocation, InViewDirection);
	}
}

void AGameplayDebuggerCategoryReplicator::ResetViewPoint()
{
	ViewLocation.Reset();
	ViewDirection.Reset();

	if (!bHasAuthority)
	{
		ServerResetViewPoint();
	}
}

void AGameplayDebuggerCategoryReplicator::SetCategoryEnabled(int32 CategoryId, bool bEnable)
{
	if (!Categories.IsValidIndex(CategoryId))
	{
		return;
	}

	if (bHasAuthority)
	{
		UE_VLOG_UELOG(this, LogGameplayDebugReplication, Log, TEXT("SetCategoryEnabled[%d]:%d (%s)"), CategoryId, bEnable ? 1 : 0, *Categories[CategoryId]->GetCategoryName().ToString());
		Categories[CategoryId]->bIsEnabled = bEnable;
	}
	else
	{
		FName CategoryName = Categories[CategoryId]->GetCategoryName();
		ServerSetCategoryNameEnabled(CategoryName, bEnable);
	}

	MarkComponentsRenderStateDirty();
}

void AGameplayDebuggerCategoryReplicator::SendCategoryInputEvent(int32 CategoryId, int32 HandlerId)
{
	if (HandlerId >= 0 && Categories.IsValidIndex(CategoryId) &&
		HandlerId < Categories[CategoryId]->GetNumInputHandlers())
	{
		// check enabled category only on local (instigating) side
		if (!bIsLocal || IsCategoryEnabled(CategoryId))
		{
			FGameplayDebuggerInputHandler& InputHandler = Categories[CategoryId]->GetInputHandler(HandlerId);
			if (InputHandler.Mode == EGameplayDebuggerInputMode::Local || bHasAuthority)
			{
				InputHandler.Delegate.ExecuteIfBound();
			}
			else
			{
				ServerSendCategoryNameInputEvent(Categories[CategoryId]->GetCategoryName(), HandlerId);
			}
		}
	}
}

void AGameplayDebuggerCategoryReplicator::SendExtensionInputEvent(int32 ExtensionId, int32 HandlerId)
{
	if (HandlerId >= 0 && Extensions.IsValidIndex(ExtensionId) &&
		HandlerId < Extensions[ExtensionId]->GetNumInputHandlers())
	{
		FGameplayDebuggerInputHandler& InputHandler = Extensions[ExtensionId]->GetInputHandler(HandlerId);
		if (InputHandler.Mode == EGameplayDebuggerInputMode::Local || bHasAuthority)
		{
			InputHandler.Delegate.ExecuteIfBound();
		}
		else
		{
			ServerSendExtensionInputEvent(ExtensionId, HandlerId);
		}
	}
}

void AGameplayDebuggerCategoryReplicator::NotifyCategoriesToolState(bool bIsActive)
{
	for (int32 Idx = 0; Idx < Categories.Num(); Idx++)
	{
		FGameplayDebuggerCategory& CategoryOb = Categories[Idx].Get();
		if (bIsActive)
		{
			CategoryOb.OnGameplayDebuggerActivated();
		}
		else
		{
			CategoryOb.OnGameplayDebuggerDeactivated();
		}
	}
}

void AGameplayDebuggerCategoryReplicator::NotifyExtensionsToolState(bool bIsActive)
{
	for (int32 Idx = 0; Idx < Extensions.Num(); Idx++)
	{
		FGameplayDebuggerExtension& ExtensionOb = Extensions[Idx].Get();
		if (bIsActive)
		{
			ExtensionOb.OnGameplayDebuggerActivated();
		}
		else
		{
			ExtensionOb.OnGameplayDebuggerDeactivated();
		}
	}
}

bool AGameplayDebuggerCategoryReplicator::IsCategoryEnabled(int32 CategoryId) const
{
	return Categories.IsValidIndex(CategoryId) && Categories[CategoryId]->IsCategoryEnabled(); 
}

