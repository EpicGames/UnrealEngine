// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayDebuggerConfig.h"
#include "GameplayDebugger.h"
#include "GameplayDebuggerAddonManager.h"
#include "CoreGlobals.h"
#include "Misc/ConfigCacheIni.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayDebuggerConfig)


UGameplayDebuggerConfig::UGameplayDebuggerConfig(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	ActivationKey = EKeys::Apostrophe;

	CategoryRowNextKey = EKeys::Add;
	CategoryRowPrevKey = EKeys::Subtract;

	CategorySlot0 = EKeys::NumPadZero;
	CategorySlot1 = EKeys::NumPadOne;
	CategorySlot2 = EKeys::NumPadTwo;
	CategorySlot3 = EKeys::NumPadThree;
	CategorySlot4 = EKeys::NumPadFour;
	CategorySlot5 = EKeys::NumPadFive;
	CategorySlot6 = EKeys::NumPadSix;
	CategorySlot7 = EKeys::NumPadSeven;
	CategorySlot8 = EKeys::NumPadEight;
	CategorySlot9 = EKeys::NumPadNine;

	DebugCanvasPaddingLeft = 10.0f;
	DebugCanvasPaddingRight = 10.0f;
	DebugCanvasPaddingTop = 10.0f;
	DebugCanvasPaddingBottom = 10.0f;
}

void UGameplayDebuggerConfig::UpdateCategoryConfig(const FName CategoryName, int32& SlotIdx, uint8& CategoryState)
{
	KnownCategoryNames.AddUnique(CategoryName);

	int32 FoundIdx = INDEX_NONE;
	for (int32 Idx = 0; Idx < Categories.Num(); Idx++)
	{
		if (*Categories[Idx].CategoryName == CategoryName)
		{
			FoundIdx = Idx;
			break;
		}
	}

	if (FoundIdx == INDEX_NONE)
	{
		FGameplayDebuggerCategoryConfig NewConfigData;
		NewConfigData.CategoryName = CategoryName.ToString();

		FoundIdx = Categories.Add(NewConfigData);
	}

	const EGameplayDebuggerCategoryState EnumState = (EGameplayDebuggerCategoryState)CategoryState;
	if (Categories.IsValidIndex(FoundIdx))
	{
		FGameplayDebuggerCategoryConfig& ConfigData = Categories[FoundIdx];
		if (ConfigData.bOverrideSlotIdx)
		{
			SlotIdx = ConfigData.SlotIdx;
		}
		else
		{
			ConfigData.SlotIdx = SlotIdx;
		}
		
		const bool bDefaultActiveInGame = (EnumState == EGameplayDebuggerCategoryState::EnabledInGame) || (EnumState == EGameplayDebuggerCategoryState::EnabledInGameAndSimulate);
		const bool bDefaultActiveInSimulate = (EnumState == EGameplayDebuggerCategoryState::EnabledInSimulate) || (EnumState == EGameplayDebuggerCategoryState::EnabledInGameAndSimulate);
		const bool bDefaultHidden = (EnumState == EGameplayDebuggerCategoryState::Hidden);
		
		const bool bActiveInGame = (ConfigData.ActiveInGame == EGameplayDebuggerOverrideMode::UseDefault) ? bDefaultActiveInGame : (ConfigData.ActiveInGame == EGameplayDebuggerOverrideMode::Enable);
		const bool bActiveInSimulate = (ConfigData.ActiveInSimulate == EGameplayDebuggerOverrideMode::UseDefault) ? bDefaultActiveInSimulate : (ConfigData.ActiveInSimulate == EGameplayDebuggerOverrideMode::Enable);
		const bool bIsHidden = (ConfigData.Hidden == EGameplayDebuggerOverrideMode::UseDefault) ? bDefaultHidden : (ConfigData.Hidden == EGameplayDebuggerOverrideMode::Enable);

		CategoryState = (uint8)(
			bIsHidden ? EGameplayDebuggerCategoryState::Hidden :
			bActiveInGame && bActiveInSimulate ? EGameplayDebuggerCategoryState::EnabledInGameAndSimulate :
			bActiveInGame ? EGameplayDebuggerCategoryState::EnabledInGame :
			bActiveInSimulate ? EGameplayDebuggerCategoryState::EnabledInSimulate :
			EGameplayDebuggerCategoryState::Disabled);
	}
}

void UGameplayDebuggerConfig::UpdateExtensionConfig(const FName ExtensionName, uint8& UseExtension)
{
	KnownExtensionNames.AddUnique(ExtensionName);

	int32 FoundIdx = INDEX_NONE;
	for (int32 Idx = 0; Idx < Extensions.Num(); Idx++)
	{
		if (*Extensions[Idx].ExtensionName == ExtensionName)
		{
			FoundIdx = Idx;
			break;
		}
	}

	if (FoundIdx == INDEX_NONE)
	{
		FGameplayDebuggerExtensionConfig NewConfigData;
		NewConfigData.ExtensionName = ExtensionName.ToString();

		FoundIdx = Extensions.Add(NewConfigData);
	}

	if (Extensions.IsValidIndex(FoundIdx))
	{
		FGameplayDebuggerExtensionConfig& ConfigData = Extensions[FoundIdx];

		const bool bDefaultUseExtension = (UseExtension != 0);
		const bool bUseExtension = (ConfigData.UseExtension == EGameplayDebuggerOverrideMode::UseDefault) ? bDefaultUseExtension : (ConfigData.UseExtension == EGameplayDebuggerOverrideMode::Enable);

		UseExtension = bUseExtension ? 1 : 0;
	}
}

void UGameplayDebuggerConfig::UpdateCategoryInputConfig(const FName CategoryName, const FName InputName, FName& KeyName, FGameplayDebuggerInputModifier& KeyModifier)
{
	int32 FoundCategoryIdx = INDEX_NONE;
	for (int32 Idx = 0; Idx < Categories.Num(); Idx++)
	{
		if (*Categories[Idx].CategoryName == CategoryName)
		{
			FoundCategoryIdx = Idx;
			break;
		}
	}

	if (!Categories.IsValidIndex(FoundCategoryIdx))
	{
		return;
	}

	FGameplayDebuggerCategoryConfig& OuterConfigData = Categories[FoundCategoryIdx];
	KnownCategoryInputNames.Add(CategoryName, InputName);

	int32 FoundIdx = INDEX_NONE;
	for (int32 Idx = 0; Idx < OuterConfigData.InputHandlers.Num(); Idx++)
	{
		if (*OuterConfigData.InputHandlers[Idx].ConfigName == InputName)
		{
			FGameplayDebuggerInputConfig& ConfigData = OuterConfigData.InputHandlers[Idx];
			KeyName = ConfigData.Key.GetFName();
			KeyModifier.bShift = ConfigData.bModShift;
			KeyModifier.bCtrl = ConfigData.bModCtrl;
			KeyModifier.bAlt = ConfigData.bModAlt;
			KeyModifier.bCmd = ConfigData.bModCmd;

			FoundIdx = Idx;
			break;
		}
	}

	if (FoundIdx == INDEX_NONE)
	{
		FGameplayDebuggerInputConfig NewConfigData;
		NewConfigData.ConfigName = InputName.ToString();
		NewConfigData.Key = FKey(KeyName);
		NewConfigData.bModShift = KeyModifier.bShift;
		NewConfigData.bModCtrl = KeyModifier.bCtrl;
		NewConfigData.bModAlt = KeyModifier.bAlt;
		NewConfigData.bModCmd = KeyModifier.bCmd;

		OuterConfigData.InputHandlers.Add(NewConfigData);
	}
}

void UGameplayDebuggerConfig::UpdateExtensionInputConfig(const FName ExtensionName, const FName InputName, FName& KeyName, FGameplayDebuggerInputModifier& KeyModifier)
{
	int32 FoundExtensionIdx = INDEX_NONE;
	for (int32 Idx = 0; Idx < Extensions.Num(); Idx++)
	{
		if (*Extensions[Idx].ExtensionName == ExtensionName)
		{
			FoundExtensionIdx = Idx;
			break;
		}
	}

	if (!Extensions.IsValidIndex(FoundExtensionIdx))
	{
		return;
	}

	FGameplayDebuggerExtensionConfig& OuterConfigData = Extensions[FoundExtensionIdx];
	KnownExtensionInputNames.Add(ExtensionName, InputName);

	int32 FoundIdx = INDEX_NONE;
	for (int32 Idx = 0; Idx < OuterConfigData.InputHandlers.Num(); Idx++)
	{
		if (*OuterConfigData.InputHandlers[Idx].ConfigName == InputName)
		{
			FGameplayDebuggerInputConfig& ConfigData = OuterConfigData.InputHandlers[Idx];
			KeyName = ConfigData.Key.GetFName();
			KeyModifier.bShift = ConfigData.bModShift;
			KeyModifier.bCtrl = ConfigData.bModCtrl;
			KeyModifier.bAlt = ConfigData.bModAlt;
			KeyModifier.bCmd = ConfigData.bModCmd;

			FoundIdx = Idx;
			break;
		}
	}

	if (FoundIdx == INDEX_NONE)
	{
		FGameplayDebuggerInputConfig NewConfigData;
		NewConfigData.ConfigName = InputName.ToString();
		NewConfigData.Key = FKey(KeyName);
		NewConfigData.bModShift = KeyModifier.bShift;
		NewConfigData.bModCtrl = KeyModifier.bCtrl;
		NewConfigData.bModAlt = KeyModifier.bAlt;
		NewConfigData.bModCmd = KeyModifier.bCmd;

		OuterConfigData.InputHandlers.Add(NewConfigData);
	}
}

void UGameplayDebuggerConfig::Serialize(FArchive& Ar)
{
	if (Ar.IsSaving())
	{
		RemoveUnknownConfigs();
	}

	Super::Serialize(Ar);
}

void UGameplayDebuggerConfig::RemoveUnknownConfigs()
{
	for (int32 Idx = Categories.Num() - 1; Idx >= 0; Idx--)
	{
		FGameplayDebuggerCategoryConfig& ConfigData = Categories[Idx];
		const bool bValid = KnownCategoryNames.Contains(*ConfigData.CategoryName);
		if (!bValid)
		{
			Categories.RemoveAt(Idx, EAllowShrinking::No);
		}
		else
		{
			for (int32 InputIdx = ConfigData.InputHandlers.Num() - 1; InputIdx >= 0; InputIdx--)
			{
				const FGameplayDebuggerInputConfig& InputConfigData = ConfigData.InputHandlers[InputIdx];
				const FName* FoundDataPtr = KnownCategoryInputNames.FindPair(*ConfigData.CategoryName, *InputConfigData.ConfigName);
				if (FoundDataPtr == nullptr)
				{
					ConfigData.InputHandlers.RemoveAt(InputIdx, EAllowShrinking::No);
				}
			}
		}
	}

	for (int32 Idx = Extensions.Num() - 1; Idx >= 0; Idx--)
	{
		FGameplayDebuggerExtensionConfig& ConfigData = Extensions[Idx];
		const bool bValid = KnownExtensionNames.Contains(*ConfigData.ExtensionName);
		if (!bValid)
		{
			Extensions.RemoveAt(Idx, EAllowShrinking::No);
		}
		else
		{
			for (int32 InputIdx = ConfigData.InputHandlers.Num() - 1; InputIdx >= 0; InputIdx--)
			{
				const FGameplayDebuggerInputConfig& InputConfigData = ConfigData.InputHandlers[InputIdx];
				const FName* FoundDataPtr = KnownExtensionInputNames.FindPair(*ConfigData.ExtensionName, *InputConfigData.ConfigName);
				if (FoundDataPtr == nullptr)
				{
					ConfigData.InputHandlers.RemoveAt(InputIdx, EAllowShrinking::No);
				}
			}
		}
	}
}

#if WITH_EDITOR
void UGameplayDebuggerConfig::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	FGameplayDebuggerAddonManager& AddonManager = FGameplayDebuggerAddonManager::GetCurrent();
	AddonManager.UpdateFromConfig();
}
#endif

//----------------------------------------------------------------------//
//  UGameplayDebuggerUserSettings
//----------------------------------------------------------------------//
void UGameplayDebuggerUserSettings::SetFontSize(const int32 InFontSize)
{
	UGameplayDebuggerUserSettings* Setting = GetMutableDefault<UGameplayDebuggerUserSettings>();
	Setting->FontSize = FMath::Clamp(InFontSize, 1, 100);

	const FString ConfigFileName = Setting->GetProjectUserConfigFilename();
	GConfig->SetInt(TEXT("/Script/GameplayDebugger.GameplayDebuggerUserSettings"), TEXT("FontSize"), Setting->FontSize, *ConfigFileName);
	GConfig->Flush(false);
}

