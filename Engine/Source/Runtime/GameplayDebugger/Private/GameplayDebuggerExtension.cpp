// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayDebuggerExtension.h"
#include "GameplayDebuggerCategoryReplicator.h"

void FGameplayDebuggerExtension::OnGameplayDebuggerActivated()
{
	const bool bIsLocal = IsLocal();
	if (bIsLocal)
	{
		OnActivated();
	}
}

void FGameplayDebuggerExtension::OnGameplayDebuggerDeactivated()
{
	const bool bIsLocal = IsLocal();
	if (bIsLocal)
	{
		OnDeactivated();
	}
}

void FGameplayDebuggerExtension::OnActivated()
{
	// empty in base class
}

void FGameplayDebuggerExtension::OnDeactivated()
{
	// empty in base class
}

FString FGameplayDebuggerExtension::GetDescription() const
{
	// empty in base class
	return FString();
}

APlayerController* FGameplayDebuggerExtension::GetPlayerController() const
{
	AGameplayDebuggerCategoryReplicator* Replicator = GetReplicator();
	return Replicator ? Replicator->GetReplicationOwner() : nullptr;
}

bool FGameplayDebuggerExtension::IsLocal() const
{
	AGameplayDebuggerCategoryReplicator* Replicator = GetReplicator();
	return Replicator ? Replicator->IsLocal() : true;
}
