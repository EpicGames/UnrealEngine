// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayDebuggerExtension_HUD.h"
#include "InputCoreTypes.h"
#include "Engine/Engine.h"
#include "GameFramework/PlayerController.h"
#include "EngineGlobals.h"
#include "GameFramework/HUD.h"

FGameplayDebuggerExtension_HUD::FGameplayDebuggerExtension_HUD()
{
	bWantsHUDEnabled = false;
	bIsGameHUDEnabled = false;
	bAreDebugMessagesEnabled = false;
	bPrevDebugMessagesEnabled = GEngine && GEngine->bEnableOnScreenDebugMessages;
	bIsCachedDescriptionValid = false;

	const FGameplayDebuggerInputHandlerConfig HUDKeyConfig(TEXT("ToggleHUD"), EKeys::Tilde.GetFName(), FGameplayDebuggerInputModifier::Ctrl);
	const FGameplayDebuggerInputHandlerConfig MessagesKeyConfig(TEXT("ToggleMessages"), EKeys::Tab.GetFName(), FGameplayDebuggerInputModifier::Ctrl);

	const bool bHasHUDBinding = BindKeyPress(HUDKeyConfig, this, &FGameplayDebuggerExtension_HUD::ToggleGameHUD);
	HudBindingIdx = bHasHUDBinding ? (GetNumInputHandlers() - 1) : INDEX_NONE;

	const bool bHasMessagesBinding = BindKeyPress(MessagesKeyConfig, this, &FGameplayDebuggerExtension_HUD::ToggleDebugMessages);
	MessagesBindingIdx = bHasMessagesBinding ? (GetNumInputHandlers() - 1) : INDEX_NONE;
}

TSharedRef<FGameplayDebuggerExtension> FGameplayDebuggerExtension_HUD::MakeInstance()
{
	return MakeShareable(new FGameplayDebuggerExtension_HUD());
}

void FGameplayDebuggerExtension_HUD::OnActivated()
{
	SetGameHUDEnabled(bWantsHUDEnabled);
	SetDebugMessagesEnabled(false);
}

void FGameplayDebuggerExtension_HUD::OnDeactivated()
{
	SetGameHUDEnabled(true);
	SetDebugMessagesEnabled(bPrevDebugMessagesEnabled);
}

FString FGameplayDebuggerExtension_HUD::GetDescription() const
{
	if (!bIsCachedDescriptionValid)
	{
		FString Description;

		if (HudBindingIdx != INDEX_NONE)
		{
			Description = FString::Printf(TEXT("{%s}%s:{%s}HUD"),
				*FGameplayDebuggerCanvasStrings::ColorNameInput,
				*GetInputHandlerDescription(HudBindingIdx),
				bIsGameHUDEnabled ? *FGameplayDebuggerCanvasStrings::ColorNameEnabled : *FGameplayDebuggerCanvasStrings::ColorNameDisabled);
		}

		if (MessagesBindingIdx != INDEX_NONE)
		{
			if (Description.Len() > 0)
			{
				Description += FGameplayDebuggerCanvasStrings::SeparatorSpace;
			}

			Description += FString::Printf(TEXT("{%s}%s:{%s}DebugMessages"),
				*FGameplayDebuggerCanvasStrings::ColorNameInput,
				*GetInputHandlerDescription(MessagesBindingIdx),
				bAreDebugMessagesEnabled ? *FGameplayDebuggerCanvasStrings::ColorNameEnabled : *FGameplayDebuggerCanvasStrings::ColorNameDisabled);
		}

		CachedDescription = Description;
		bIsCachedDescriptionValid = true;
	}

	return CachedDescription;
}

void FGameplayDebuggerExtension_HUD::SetGameHUDEnabled(bool bEnable)
{
	APlayerController* OwnerPC = GetPlayerController();
	AHUD* GameHUD = OwnerPC ? OwnerPC->GetHUD() : nullptr;
	if (GameHUD)
	{
		// ShowHUD is actually a toggle and is a UFUNCTION(exec).
		// We make sure the flag is !bEnable before calling the function so we don't end up desynced.
		// Also, we don't simply set the flag to bEnable because we want to execute functions that overrides ShowHUD.
		GameHUD->bShowHUD = !bEnable;
		GameHUD->ShowHUD();
	}

	bIsGameHUDEnabled = bEnable;
	bIsCachedDescriptionValid = false;
}

void FGameplayDebuggerExtension_HUD::SetDebugMessagesEnabled(bool bEnable)
{
	if (GEngine)
	{
		GEngine->bEnableOnScreenDebugMessages = bEnable;
	}

	bAreDebugMessagesEnabled = bEnable;
	bIsCachedDescriptionValid = false;
}

void FGameplayDebuggerExtension_HUD::ToggleGameHUD()
{
	const bool bEnable = bWantsHUDEnabled = !bIsGameHUDEnabled;
	SetGameHUDEnabled(bEnable);
}

void FGameplayDebuggerExtension_HUD::ToggleDebugMessages()
{
	SetDebugMessagesEnabled(!bAreDebugMessagesEnabled);
}
