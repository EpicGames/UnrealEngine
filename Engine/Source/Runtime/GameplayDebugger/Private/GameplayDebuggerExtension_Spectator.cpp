// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayDebuggerExtension_Spectator.h"
#include "InputCoreTypes.h"
#include "Engine/World.h"
#include "GameplayDebuggerPlayerManager.h"
#include "Engine/DebugCameraController.h"
#include "Engine/Player.h"
#include "GameFramework/HUD.h"
#include "GameFramework/WorldSettings.h"

FGameplayDebuggerExtension_Spectator::FGameplayDebuggerExtension_Spectator()
{
	const FGameplayDebuggerInputHandlerConfig KeyConfig(TEXT("Toggle"), EKeys::Tab.GetFName());
	bHasInputBinding = BindKeyPress(KeyConfig, this, &FGameplayDebuggerExtension_Spectator::ToggleSpectatorMode);
}

TSharedRef<FGameplayDebuggerExtension> FGameplayDebuggerExtension_Spectator::MakeInstance()
{
	return MakeShareable(new FGameplayDebuggerExtension_Spectator());
}

void FGameplayDebuggerExtension_Spectator::OnDeactivated()
{
	if (SpectatorController.IsValid())
	{
		ToggleSpectatorMode();
	}
}

FString FGameplayDebuggerExtension_Spectator::GetDescription() const
{
	if (!bIsCachedDescriptionValid)
	{
		CachedDescription = !bHasInputBinding ? FString() :
			FString::Printf(TEXT("{%s}%s:{%s}Spectator"),
				*FGameplayDebuggerCanvasStrings::ColorNameInput,
				*GetInputHandlerDescription(0),
				SpectatorController.IsValid() ? *FGameplayDebuggerCanvasStrings::ColorNameEnabled : *FGameplayDebuggerCanvasStrings::ColorNameDisabled);

		bIsCachedDescriptionValid = true;
	}

	return CachedDescription;
}

void FGameplayDebuggerExtension_Spectator::ToggleSpectatorMode()
{
	APlayerController* OwnerPC = GetPlayerController();
	AGameplayDebuggerPlayerManager& PlayerManager = AGameplayDebuggerPlayerManager::GetCurrent(OwnerPC->GetWorld());
	UInputComponent* DebuggerInput = PlayerManager.GetInputComponent(*OwnerPC);

	ADebugCameraController* SpectatorControllerOb = SpectatorController.Get();
	if (SpectatorControllerOb == nullptr)
	{
		FActorSpawnParameters SpawnInfo;
		SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
		SpawnInfo.Owner = OwnerPC->GetWorldSettings();
		SpawnInfo.Instigator = OwnerPC->GetInstigator();
		SpectatorControllerOb = OwnerPC->GetWorld()->SpawnActor<ADebugCameraController>(SpawnInfo);

		if (SpectatorControllerOb)
		{
			OwnerPC->PopInputComponent(DebuggerInput);
			SpectatorControllerOb->OnActivate(OwnerPC);
			OwnerPC->Player->SwitchController(SpectatorControllerOb);
			SpectatorControllerOb->PushInputComponent(DebuggerInput);

			SpectatorControllerOb->ChangeState(NAME_Default);
			SpectatorControllerOb->ChangeState(NAME_Spectating);
			
			if (SpectatorControllerOb->MyHUD)
			{
				SpectatorControllerOb->MyHUD->bShowHUD = false;
			}

			SpectatorController = SpectatorControllerOb;
		}
	}
	else
	{
		SpectatorControllerOb->PopInputComponent(DebuggerInput);
		if (SpectatorControllerOb->OriginalPlayer)
		{
			SpectatorControllerOb->OriginalPlayer->SwitchController(OwnerPC);
		}
		SpectatorControllerOb->OnDeactivate(OwnerPC);
		OwnerPC->PushInputComponent(DebuggerInput);

		SpectatorControllerOb->GetWorld()->DestroyActor(SpectatorControllerOb, false, false);
		SpectatorController = nullptr;
	}

	bIsCachedDescriptionValid = false;
}
