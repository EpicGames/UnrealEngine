// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayDebuggerLocalController.h"
#include "Engine/Engine.h"
#include "InputCoreTypes.h"
#include "Framework/Commands/InputChord.h"
#include "Components/InputComponent.h"
#include "Misc/App.h"
#include "SceneView.h"
#include "TimerManager.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/Controller.h"
#include "GameFramework/SpectatorPawn.h"
#include "GameplayDebuggerTypes.h"
#include "GameplayDebuggerCategoryReplicator.h"
#include "GameplayDebuggerPlayerManager.h"
#include "GameplayDebuggerAddonBase.h"
#include "GameplayDebuggerCategory.h"
#include "GameplayDebuggerAddonManager.h"
#include "GameplayDebuggerExtension.h"
#include "GameplayDebuggerConfig.h"
#include "GameplayDebuggerModule.h"
#include "Debug/DebugDrawService.h"
#include "Engine/Selection.h"
#include "CanvasItem.h"
#include "Engine/Canvas.h"
#include "Engine/DebugCameraController.h"
#include "GameFramework/PlayerInput.h"
#include "EngineUtils.h"
#include "HAL/IConsoleManager.h"
#include "SceneInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayDebuggerLocalController)

#if WITH_EDITOR
#include "Editor.h"
#endif // WITH_EDITOR

#if WITH_GAMEPLAY_DEBUGGER
bool UGameplayDebuggerLocalController::bConsoleCommandsEnabled = true;
#else
bool UGameplayDebuggerLocalController::bConsoleCommandsEnabled = false;
#endif

UGameplayDebuggerLocalController::UGameplayDebuggerLocalController(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	bSimulateMode = false;
	bNeedsCleanup = false;
	bIsSelectingActor = false;
	bIsLocallyEnabled = false;
	bPrevLocallyEnabled = false;
	bEnableTextShadow = false;
	bPrevScreenMessagesEnabled = false;
#if WITH_EDITOR
	bActivateOnPIEEnd = false;
#endif // WITH_EDITOR

#if WITH_GAMEPLAY_DEBUGGER_MENU
	ActiveRowIdx = 0;
	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		HUDFont = NewObject<UFont>(this, TEXT("HUDFont"), RF_NoFlags, GEngine->GetSmallFont());
		HUDFont->LegacyFontSize = UGameplayDebuggerUserSettings::GetFontSize(); //FGameplayDebuggerTweakables::FontSize;
	}
#endif // WITH_GAMEPLAY_DEBUGGER_MENU
}

void UGameplayDebuggerLocalController::Initialize(AGameplayDebuggerCategoryReplicator& Replicator, AGameplayDebuggerPlayerManager& Manager)
{
	CachedReplicator = &Replicator;
	CachedPlayerManager = &Manager;
	bSimulateMode = FGameplayDebuggerAddonBase::IsSimulateInEditor() || Replicator.IsEditorWorldReplicator();

#if WITH_GAMEPLAY_DEBUGGER_MENU
	UDebugDrawService::Register(bSimulateMode ? TEXT("DebugAI") : TEXT("Game"), FDebugDrawDelegate::CreateUObject(this, &UGameplayDebuggerLocalController::OnDebugDraw));

#if WITH_EDITOR
	if (bSimulateMode)
	{
		FGameplayDebuggerModule::OnLocalControllerInitialized.Broadcast();		
	}

	if (GIsEditor)
	{
		USelection::SelectionChangedEvent.AddUObject(this, &UGameplayDebuggerLocalController::OnSelectionChanged);
		USelection::SelectObjectEvent.AddUObject(this, &UGameplayDebuggerLocalController::OnSelectedObject);

		if (Replicator.IsEditorWorldReplicator())
		{
			// bind to PIE start and end notifies to hide before pie and re-enable if need be when pie's done
			FEditorDelegates::BeginPIE.AddUObject(this, &UGameplayDebuggerLocalController::OnBeginPIE);
			FEditorDelegates::EndPIE.AddUObject(this, &UGameplayDebuggerLocalController::OnEndPIE);
		}
	}
#endif

	const UGameplayDebuggerConfig* SettingsCDO = UGameplayDebuggerConfig::StaticClass()->GetDefaultObject<UGameplayDebuggerConfig>();
	const FKey NumpadKeys[] = { EKeys::NumPadZero, EKeys::NumPadOne, EKeys::NumPadTwo, EKeys::NumPadThree, EKeys::NumPadFour,
		EKeys::NumPadFive, EKeys::NumPadSix, EKeys::NumPadSeven, EKeys::NumPadEight, EKeys::NumPadNine };
	const FKey CategorySlots[] = { SettingsCDO->CategorySlot0, SettingsCDO->CategorySlot1, SettingsCDO->CategorySlot2, SettingsCDO->CategorySlot3, SettingsCDO->CategorySlot4,
		SettingsCDO->CategorySlot5, SettingsCDO->CategorySlot6, SettingsCDO->CategorySlot7, SettingsCDO->CategorySlot8, SettingsCDO->CategorySlot9 };

	bool bIsNumpadOnly = true;
	for (int32 Idx = 0; Idx < UE_ARRAY_COUNT(CategorySlots); Idx++)
	{
		bool bHasPattern = false;
		for (int32 PatternIdx = 0; PatternIdx < UE_ARRAY_COUNT(NumpadKeys); PatternIdx++)
		{
			if (CategorySlots[Idx] == NumpadKeys[PatternIdx])
			{
				bHasPattern = true;
				break;
			}
		}

		if (!bHasPattern)
		{
			bIsNumpadOnly = false;
			break;
		}
	}

	ActivationKeyDesc = GetKeyDescriptionLong(SettingsCDO->ActivationKey);
	RowUpKeyDesc = GetKeyDescriptionShort(SettingsCDO->CategoryRowPrevKey);
	RowDownKeyDesc = GetKeyDescriptionShort(SettingsCDO->CategoryRowNextKey);
	CategoryKeysDesc = bIsNumpadOnly ? TEXT("{yellow}Numpad{white}") : TEXT("highlighted keys");

	PaddingLeft = SettingsCDO->DebugCanvasPaddingLeft;
	PaddingRight = SettingsCDO->DebugCanvasPaddingRight;
	PaddingTop = SettingsCDO->DebugCanvasPaddingTop;
	PaddingBottom = SettingsCDO->DebugCanvasPaddingBottom;

	bEnableTextShadow = SettingsCDO->bDebugCanvasEnableTextShadow;
#endif // WITH_GAMEPLAY_DEBUGGER_MENU

	FGameplayDebuggerAddonManager& AddonManager = FGameplayDebuggerAddonManager::GetCurrent();
	AddonManager.OnCategoriesChanged.AddUObject(this, &UGameplayDebuggerLocalController::OnCategoriesChanged);
	OnCategoriesChanged();

	// Restore shown categories from last session
	RestoreCategories();

	bNeedsCleanup = true;
}

void UGameplayDebuggerLocalController::Cleanup()
{
#if WITH_EDITOR
	USelection::SelectionChangedEvent.RemoveAll(this);
	USelection::SelectObjectEvent.RemoveAll(this);

	if (bSimulateMode)
	{
		FGameplayDebuggerModule::OnLocalControllerUninitialized.Broadcast();
	}
#endif // WITH_EDITOR

	// If we are cleaning up while enabled, restore the screen messages flag
	if (bIsLocallyEnabled && !GAreScreenMessagesEnabled)
	{
		GAreScreenMessagesEnabled = bPrevScreenMessagesEnabled;
	}

	bNeedsCleanup = false;
}

void UGameplayDebuggerLocalController::BeginDestroy()
{
	Super::BeginDestroy();
	if (bNeedsCleanup)
	{
		Cleanup();
	}
}

void UGameplayDebuggerLocalController::OnCategoriesChanged()
{
	FGameplayDebuggerAddonManager& AddonManager = FGameplayDebuggerAddonManager::GetCurrent();

	SlotNames.Reset();
	SlotNames.Append(AddonManager.GetSlotNames());

	// categories are already sorted using AddonManager.SlotMap, build Slot to Category Id map accordingly
	const TArray< TArray<int32> >& SlotMap = AddonManager.GetSlotMap();
	SlotCategoryIds.Reset();
	SlotCategoryIds.AddDefaulted(SlotMap.Num());

	int32 CategoryId = 0;
	for (int32 SlotIdx = 0; SlotIdx < SlotMap.Num(); SlotIdx++)
	{
		for (int32 InnerIdx = 0; InnerIdx < SlotMap[SlotIdx].Num(); InnerIdx++)
		{
			SlotCategoryIds[SlotIdx].Add(CategoryId);
			CategoryId++;
		}
	}

	NumCategorySlots = SlotCategoryIds.Num();
	NumCategories = AddonManager.GetNumVisibleCategories();

	DataPackMap.Reset();
}

void UGameplayDebuggerLocalController::RestoreCategories()
{
	// Retrieve from config which categories were enabled last session
	const TSet<FName>& EnabledCategories = GetDefault<UGameplayDebuggerUserSettings>()->EnabledCategories;

	// If saved category list is empty, use the category's default visibility. This is when 
	// the user has never toggled a category or if they turned them all off.
	if (EnabledCategories.IsEmpty())
	{
		return;
	}

	// Restore enabled and disabled categories from last session
	const int32 NumReplicatorCategories = CachedReplicator->GetNumCategories();
	for (int32 i = 0; i < NumReplicatorCategories; ++i)
	{
		// Get the category's name: this is saved to disk since id's are unstable
		const TSharedRef<FGameplayDebuggerCategory>& Category = CachedReplicator->GetCategory(i);
		const FName CategoryName = Category->GetCategoryName();

		// Restore the category's enabled state
		const bool bWasEnabledLastSession = EnabledCategories.Contains(CategoryName);
		if (bWasEnabledLastSession != CachedReplicator->IsCategoryEnabled(i))
		{
			CachedReplicator->SetCategoryEnabled(i, bWasEnabledLastSession);
		}
	}
}

#if WITH_GAMEPLAY_DEBUGGER_MENU
void UGameplayDebuggerLocalController::OnDebugDraw(UCanvas* Canvas, APlayerController* PlayerController)
{
	check(Canvas);
	if (CachedReplicator == nullptr || CachedReplicator->IsEnabled() == false || bDebugDrawEnabled == false)
	{
		return;
	}

	// Draw on canvas only when OnDebugDraw (bound to global multicast delegate; no world differentiation)
	// is called without a specific player controller or with the one associated to our replicator.
	// This validation is required for multiplayer PIE, including split-screen.
	if (PlayerController != nullptr && CachedReplicator->GetReplicationOwner() != PlayerController)
	{
		return;
	}

	FGameplayDebuggerCanvasContext CanvasContext(Canvas, HUDFont);
	CanvasContext.CursorX = CanvasContext.DefaultX = PaddingLeft;
	CanvasContext.CursorY = CanvasContext.DefaultY = PaddingTop;

	CanvasContext.FontRenderInfo.bEnableShadow = bEnableTextShadow;

	APlayerController* ReplicationOwner = CachedReplicator->GetReplicationOwner();
	CanvasContext.PlayerController = ReplicationOwner;
	CanvasContext.World = ReplicationOwner ? ReplicationOwner->GetWorld() : CachedReplicator->GetWorld();

	DrawHeader(CanvasContext);

	if (DataPackMap.Num() != NumCategories)
	{
		RebuildDataPackMap();
	}

	if (!Canvas->SceneView->ViewActor.IsSet())
	{
		CachedReplicator->SetViewPoint(Canvas->SceneView->ViewLocation, Canvas->SceneView->ViewRotation.Vector());
	}
	else if (CachedReplicator->IsViewPointSet())
	{
		CachedReplicator->ResetViewPoint();
	}

	const bool bHasDebugActor = CachedReplicator->HasDebugActor();
	for (int32 Idx = 0; Idx < NumCategories; Idx++)
	{
		TSharedRef<FGameplayDebuggerCategory> Category = CachedReplicator->GetCategory(Idx);
		if (Category->ShouldDrawCategory(bHasDebugActor))
		{
			// this is a special-case collection mode. If we want to collect data on the client this is the 
			// place to do it, after the data got potentially replicated over from the server, and just 
			// before drawing, so that new replicated data won't come in as we draw.
			if ((Category->IsCategoryAuth() == false) && Category->ShouldCollectDataOnClient())
			{
				Category->CollectData(ReplicationOwner, CachedReplicator->GetDebugActor());
			}

			if (Category->IsCategoryHeaderVisible())
			{
				DrawCategoryHeader(Idx, Category, CanvasContext);
			}

			Category->DrawCategory(ReplicationOwner, CanvasContext);
		}
	}
}

extern RENDERCORE_API FTexture* GWhiteTexture;

void UGameplayDebuggerLocalController::DrawHeader(FGameplayDebuggerCanvasContext& CanvasContext)
{
	const int32 NumRows = (NumCategorySlots + (NumCategoriesPerRow-1)) / NumCategoriesPerRow;
	const float LineHeight = CanvasContext.GetLineHeight();
	const int32 NumExtensions = bSimulateMode ? 0 : CachedReplicator->GetNumExtensions();
	const int32 NumExtensionRows = (NumExtensions > 0) ? 1 : 0;
	const float DPIScale = CanvasContext.Canvas->GetDPIScale();
	const float CanvasSizeX = (CanvasContext.Canvas->SizeX / DPIScale) - PaddingLeft - PaddingRight;
	const float UsePaddingTop = PaddingTop + (bSimulateMode ? 30.0f : 0);
	
	const float BackgroundPadding = 5.0f;
	const float BackgroundPaddingBothSides = BackgroundPadding * 2.0f;

	if (NumRows > 1)
	{
		FCanvasTileItem TileItemUpper(FVector2D(0, 0), GWhiteTexture, FVector2D(CanvasSizeX + BackgroundPaddingBothSides, (LineHeight * (ActiveRowIdx + NumExtensionRows + 1)) + BackgroundPadding), FLinearColor(0, 0, 0, 0.2f));
		FCanvasTileItem ActiveRowTileItem(FVector2D(0, 0), GWhiteTexture, FVector2D(CanvasSizeX + BackgroundPaddingBothSides, LineHeight), FLinearColor(0, 0.5f, 0, 0.3f));
		FCanvasTileItem TileItemLower(FVector2D(0, 0), GWhiteTexture, FVector2D(CanvasSizeX + BackgroundPaddingBothSides, LineHeight * ((NumRows - ActiveRowIdx - 1)) + BackgroundPadding), FLinearColor(0, 0, 0, 0.2f));

		TileItemUpper.BlendMode = SE_BLEND_Translucent;
		ActiveRowTileItem.BlendMode = SE_BLEND_Translucent;
		TileItemLower.BlendMode = SE_BLEND_Translucent;

		CanvasContext.DrawItem(TileItemUpper, PaddingLeft - BackgroundPadding, UsePaddingTop - BackgroundPadding);
		CanvasContext.DrawItem(ActiveRowTileItem, PaddingLeft - BackgroundPadding, UsePaddingTop - BackgroundPadding + TileItemUpper.Size.Y);
		CanvasContext.DrawItem(TileItemLower, PaddingLeft - BackgroundPadding, UsePaddingTop - BackgroundPadding + TileItemUpper.Size.Y + ActiveRowTileItem.Size.Y);
	}
	else
	{
		FCanvasTileItem TileItem(FVector2D(0, 0), GWhiteTexture, FVector2D(CanvasSizeX, LineHeight * (NumRows + NumExtensionRows + 1)) + BackgroundPaddingBothSides, FLinearColor(0, 0, 0, 0.2f));
		TileItem.BlendMode = SE_BLEND_Translucent;
		CanvasContext.DrawItem(TileItem, PaddingLeft - BackgroundPadding, UsePaddingTop - BackgroundPadding);
	}

	CanvasContext.CursorY = UsePaddingTop;
	if (bSimulateMode)
	{
		CanvasContext.Printf(TEXT("Clear {yellow}DebugAI{white} show flag to close, use %s to toggle categories."), *CategoryKeysDesc);

		// reactivate editor mode when this is being drawn = show flag is set
#if WITH_EDITOR
	FGameplayDebuggerModule::OnDebuggerEdModeActivation.Broadcast();
#endif // WITH_EDITOR
	}
	else
	{
		const UGameplayDebuggerConfig* Config = UGameplayDebuggerConfig::StaticClass()->GetDefaultObject<UGameplayDebuggerConfig>();
		CanvasContext.Printf(TEXT("Tap {yellow}%s{white} to close or hold to select new Pawn (hold {yellow}[Shift+%s]{white} to select local player). Use %s to toggle categories."),
			*ActivationKeyDesc,
			*Config->ActivationKey.ToString(),
			*CategoryKeysDesc);
	}

	// Get the NetRole string so we can hint if the user has selected a local Actor or not
	const AActor* DebugActor = CachedReplicator ? CachedReplicator->GetDebugActor() : nullptr;
	const ENetRole DebugNetRole = DebugActor ? DebugActor->GetLocalRole() : ENetRole::ROLE_None;
	const FString DebugNetRoleString = UEnum::GetValueAsString<ENetRole>(DebugNetRole);
	
	const FString DebugActorDesc = FString::Printf(TEXT("Debug actor: {cyan}%s{white} [%s]"), *CachedReplicator->GetDebugActorName().ToString(), *DebugNetRoleString);
	float DebugActorSizeX = 0.0f, DebugActorSizeY = 0.0f;
	CanvasContext.MeasureString(DebugActorDesc, DebugActorSizeX, DebugActorSizeY);
	CanvasContext.PrintAt((CanvasContext.Canvas->SizeX / DPIScale) - PaddingRight - DebugActorSizeX, UsePaddingTop, DebugActorDesc);

	const FString VLogDesc = FString::Printf(TEXT("VLog: {cyan}%s"), CachedReplicator->GetVisLogSyncData().DeviceIDs.Len() > 0
			? *CachedReplicator->GetVisLogSyncData().DeviceIDs
			: TEXT("not recording to file"));
		float VLogSizeX = 0.0f, VLogSizeY = 0.0f;
		CanvasContext.MeasureString(VLogDesc, VLogSizeX, VLogSizeY);
		CanvasContext.PrintAt((CanvasContext.Canvas->SizeX / DPIScale) - PaddingRight - VLogSizeX, UsePaddingTop + LineHeight, VLogDesc);

	const FString TimestampDesc = FString::Printf(TEXT("Time: %.2fs"), CachedReplicator->GetWorld()->GetTimeSeconds());
	float TimestampSizeX = 0.0f, TimestampSizeY = 0.0f;
	CanvasContext.MeasureString(TimestampDesc, TimestampSizeX, TimestampSizeY);
	CanvasContext.PrintAt((CanvasSizeX - TimestampSizeX) * 0.5f, UsePaddingTop, TimestampDesc);

	if (NumRows > 1)
	{
		const FString ChangeRowDesc = FString::Printf(TEXT("Prev row: {yellow}%s\n{white}Next row: {yellow}%s"), *RowUpKeyDesc, *RowDownKeyDesc);
		float RowDescSizeX = 0.0f, RowDescSizeY = 0.0f;
		CanvasContext.MeasureString(ChangeRowDesc, RowDescSizeX, RowDescSizeY);
		CanvasContext.PrintAt((CanvasContext.Canvas->SizeX / DPIScale) - PaddingRight - RowDescSizeX, UsePaddingTop + LineHeight * (NumExtensionRows + 1), ChangeRowDesc);
	}

	if (NumExtensionRows)
	{
		FString ExtensionRowDesc;
		for (int32 ExtensionIdx = 0; ExtensionIdx < NumExtensions; ExtensionIdx++)
		{
			TSharedRef<FGameplayDebuggerExtension> Extension = CachedReplicator->GetExtension(ExtensionIdx);
			FString ExtensionDesc = Extension->GetDescription();
			ExtensionDesc.ReplaceInline(TEXT("\n"), TEXT(""));

			if (ExtensionDesc.Len())
			{
				if (ExtensionRowDesc.Len())
				{
					ExtensionRowDesc += FGameplayDebuggerCanvasStrings::SeparatorSpace;
				}

				ExtensionRowDesc += ExtensionDesc;
			}
		}

		CanvasContext.Print(ExtensionRowDesc);
	}

	for (int32 RowIdx = 0; RowIdx < NumRows; RowIdx++)
	{
		FString CategoryRowDesc;
		for (int32 Idx = 0; Idx < NumCategoriesPerRow; Idx++)
		{
			const int32 CategorySlotIdx = (RowIdx * NumCategoriesPerRow) + Idx;
			if (SlotCategoryIds.IsValidIndex(CategorySlotIdx) && 
				SlotNames.IsValidIndex(CategorySlotIdx) &&
				SlotCategoryIds[CategorySlotIdx].Num())
			{
				TSharedRef<FGameplayDebuggerCategory> Category0 = CachedReplicator->GetCategory(SlotCategoryIds[CategorySlotIdx][0]);
				const bool bIsEnabled = Category0->IsCategoryEnabled();
				const FString CategoryColorName = (RowIdx == ActiveRowIdx) && (NumRows > 1) ?
					(bIsEnabled ? *FGameplayDebuggerCanvasStrings::ColorNameEnabledActiveRow : *FGameplayDebuggerCanvasStrings::ColorNameDisabledActiveRow) :
					(bIsEnabled ? *FGameplayDebuggerCanvasStrings::ColorNameEnabled : *FGameplayDebuggerCanvasStrings::ColorNameDisabled);

				const FString CategoryDesc = (RowIdx == ActiveRowIdx) ?
					FString::Printf(TEXT("%s{%s}%d:{%s}%s"),
						Idx ? *FGameplayDebuggerCanvasStrings::SeparatorSpace : TEXT(""),
						*FGameplayDebuggerCanvasStrings::ColorNameInput,
						Idx,
						*CategoryColorName,
						*SlotNames[CategorySlotIdx]) :
					FString::Printf(TEXT("%s{%s}%s"),
						Idx ? *FGameplayDebuggerCanvasStrings::Separator : TEXT(""),
						*CategoryColorName,
						*SlotNames[CategorySlotIdx]);

				CategoryRowDesc += CategoryDesc;
			}
		}

		CanvasContext.Print(CategoryRowDesc);
	}

	CanvasContext.DefaultY = CanvasContext.CursorY + LineHeight;
}

void UGameplayDebuggerLocalController::DrawCategoryHeader(int32 CategoryId, TSharedRef<FGameplayDebuggerCategory> Category, FGameplayDebuggerCanvasContext& CanvasContext)
{
	FString DataPackDesc;
	
	if (DataPackMap.IsValidIndex(CategoryId) &&
		!Category->IsCategoryAuth() &&
		!Category->ShouldDrawReplicationStatus() &&
		Category->GetNumDataPacks() > 0)
	{
		// collect brief data pack status, detailed info is displayed only when ShouldDrawReplicationStatus is true
		const int16 CurrentSyncCounter = CachedReplicator->GetDebugActorCounter();

		DataPackDesc = TEXT("{white} ver[");
		bool bIsPrevOutdated = false;
		bool bAddSeparator = false;

		for (int32 Idx = 0; Idx < DataPackMap[CategoryId].Num(); Idx++)
		{
			TSharedRef<FGameplayDebuggerCategory> MappedCategory = CachedReplicator->GetCategory(DataPackMap[CategoryId][Idx]);
			for (int32 DataPackIdx = 0; DataPackIdx < MappedCategory->GetNumDataPacks(); DataPackIdx++)
			{
				FGameplayDebuggerDataPack::FHeader DataHeader = MappedCategory->GetDataPackHeaderCopy(DataPackIdx);
				const bool bIsOutdated = (DataHeader.SyncCounter != CurrentSyncCounter);

				if (bAddSeparator)
				{
					DataPackDesc += TEXT(';');
				}

				if (bIsOutdated != bIsPrevOutdated)
				{
					DataPackDesc += bIsOutdated ? TEXT("{red}") : TEXT("{white}");
					bIsPrevOutdated = bIsOutdated;
				}

				DataPackDesc += TTypeToString<int16>::ToString(DataHeader.DataVersion);
				bAddSeparator = true;
			}
		}

		if (bIsPrevOutdated)
		{
			DataPackDesc += TEXT("{white}");
		}

		DataPackDesc += TEXT(']');
	}

	CanvasContext.MoveToNewLine();
	CanvasContext.Printf(FColor::Green, TEXT("[CATEGORY: %s]%s"), *Category->GetCategoryName().ToString(), *DataPackDesc);
}

void UGameplayDebuggerLocalController::OnSelectLocalPlayer()
{
	APlayerController* OwnerPC = CachedReplicator ? CachedReplicator->GetReplicationOwner().Get() : nullptr;
	// Normal game. Spectator pawns aren't considered a valid local player to debug
	if (OwnerPC && OwnerPC->Player)
	{
		if (AActor* LocalPlayerActor = OwnerPC->GetPawn())
		{
			CachedReplicator->SetDebugActor(LocalPlayerActor, true);
			CachedReplicator->CollectCategoryData(/*bForce=*/true);
		}
	}
}

void UGameplayDebuggerLocalController::BindInput(UInputComponent& InputComponent)
{
	TSet<FName> NewBindings;

	const UGameplayDebuggerConfig* SettingsCDO = UGameplayDebuggerConfig::StaticClass()->GetDefaultObject<UGameplayDebuggerConfig>();
	if (!bSimulateMode)
	{
		InputComponent.BindKey(FInputChord(EModifierKey::None, SettingsCDO->ActivationKey), IE_Pressed, this, &UGameplayDebuggerLocalController::OnActivationPressed);
		InputComponent.BindKey(FInputChord(EModifierKey::None, SettingsCDO->ActivationKey), IE_Released, this, &UGameplayDebuggerLocalController::OnActivationReleased);
		InputComponent.BindKey(FInputChord(EModifierKey::Shift, SettingsCDO->ActivationKey), IE_Pressed, this, &UGameplayDebuggerLocalController::OnActivationPressedWithModifier);
		InputComponent.BindKey(FInputChord(EModifierKey::Shift, SettingsCDO->ActivationKey), IE_Released, this, &UGameplayDebuggerLocalController::OnActivationReleasedWithModifier);

		NewBindings.Add(SettingsCDO->ActivationKey.GetFName());
	}

	if (bIsLocallyEnabled || bSimulateMode)
	{
		InputComponent.BindKey(SettingsCDO->CategorySlot0, IE_Pressed, this, &UGameplayDebuggerLocalController::OnCategory0Pressed);
		InputComponent.BindKey(SettingsCDO->CategorySlot1, IE_Pressed, this, &UGameplayDebuggerLocalController::OnCategory1Pressed);
		InputComponent.BindKey(SettingsCDO->CategorySlot2, IE_Pressed, this, &UGameplayDebuggerLocalController::OnCategory2Pressed);
		InputComponent.BindKey(SettingsCDO->CategorySlot3, IE_Pressed, this, &UGameplayDebuggerLocalController::OnCategory3Pressed);
		InputComponent.BindKey(SettingsCDO->CategorySlot4, IE_Pressed, this, &UGameplayDebuggerLocalController::OnCategory4Pressed);
		InputComponent.BindKey(SettingsCDO->CategorySlot5, IE_Pressed, this, &UGameplayDebuggerLocalController::OnCategory5Pressed);
		InputComponent.BindKey(SettingsCDO->CategorySlot6, IE_Pressed, this, &UGameplayDebuggerLocalController::OnCategory6Pressed);
		InputComponent.BindKey(SettingsCDO->CategorySlot7, IE_Pressed, this, &UGameplayDebuggerLocalController::OnCategory7Pressed);
		InputComponent.BindKey(SettingsCDO->CategorySlot8, IE_Pressed, this, &UGameplayDebuggerLocalController::OnCategory8Pressed);
		InputComponent.BindKey(SettingsCDO->CategorySlot9, IE_Pressed, this, &UGameplayDebuggerLocalController::OnCategory9Pressed);

		InputComponent.BindKey(SettingsCDO->CategoryRowPrevKey, IE_Pressed, this, &UGameplayDebuggerLocalController::OnCategoryRowUpPressed);
		InputComponent.BindKey(SettingsCDO->CategoryRowNextKey, IE_Pressed, this, &UGameplayDebuggerLocalController::OnCategoryRowDownPressed);

		NewBindings.Add(SettingsCDO->CategorySlot0.GetFName());
		NewBindings.Add(SettingsCDO->CategorySlot1.GetFName());
		NewBindings.Add(SettingsCDO->CategorySlot2.GetFName());
		NewBindings.Add(SettingsCDO->CategorySlot3.GetFName());
		NewBindings.Add(SettingsCDO->CategorySlot4.GetFName());
		NewBindings.Add(SettingsCDO->CategorySlot5.GetFName());
		NewBindings.Add(SettingsCDO->CategorySlot6.GetFName());
		NewBindings.Add(SettingsCDO->CategorySlot7.GetFName());
		NewBindings.Add(SettingsCDO->CategorySlot8.GetFName());
		NewBindings.Add(SettingsCDO->CategorySlot9.GetFName());
		NewBindings.Add(SettingsCDO->CategoryRowPrevKey.GetFName());
		NewBindings.Add(SettingsCDO->CategoryRowNextKey.GetFName());

		for (int32 Idx = 0; Idx < NumCategories; Idx++)
		{
			TSharedRef<FGameplayDebuggerCategory> Category = CachedReplicator->GetCategory(Idx);
			const int32 NumInputHandlers = Category->GetNumInputHandlers();

			for (int32 HandlerIdx = 0; HandlerIdx < NumInputHandlers; HandlerIdx++)
			{
				FGameplayDebuggerInputHandler& HandlerData = Category->GetInputHandler(HandlerIdx);
				if (HandlerData.Modifier.bPressed || HandlerData.Modifier.bReleased)
				{
					FInputChord InputChord(FKey(HandlerData.KeyName), HandlerData.Modifier.bShift, HandlerData.Modifier.bCtrl, HandlerData.Modifier.bAlt, HandlerData.Modifier.bCmd);
					FInputKeyBinding InputBinding(InputChord, HandlerData.Modifier.bPressed ? IE_Pressed : IE_Released);
					InputBinding.KeyDelegate.GetDelegateForManualSet().BindUObject(this, &UGameplayDebuggerLocalController::OnCategoryBindingEvent, Idx, HandlerIdx);

					InputComponent.KeyBindings.Add(InputBinding);
					NewBindings.Add(HandlerData.KeyName);
				}
			}
		}

		const int32 NumExtentions = bSimulateMode ? 0 : CachedReplicator->GetNumExtensions();
		for (int32 Idx = 0; Idx < NumExtentions; Idx++)
		{
			TSharedRef<FGameplayDebuggerExtension> Extension = CachedReplicator->GetExtension(Idx); //-V595
			const int32 NumInputHandlers = Extension->GetNumInputHandlers();

			for (int32 HandlerIdx = 0; HandlerIdx < NumInputHandlers; HandlerIdx++)
			{
				FGameplayDebuggerInputHandler& HandlerData = Extension->GetInputHandler(HandlerIdx);
				if (HandlerData.Modifier.bPressed || HandlerData.Modifier.bReleased)
				{
					FInputChord InputChord(FKey(HandlerData.KeyName), HandlerData.Modifier.bShift, HandlerData.Modifier.bCtrl, HandlerData.Modifier.bAlt, HandlerData.Modifier.bCmd);
					FInputKeyBinding InputBinding(InputChord, HandlerData.Modifier.bPressed ? IE_Pressed : IE_Released);
					InputBinding.KeyDelegate.GetDelegateForManualSet().BindUObject(this, &UGameplayDebuggerLocalController::OnExtensionBindingEvent, Idx, HandlerIdx);

					InputComponent.KeyBindings.Add(InputBinding);
					NewBindings.Add(HandlerData.KeyName);
				}
			}
		}
	}

	if (CachedReplicator && CachedReplicator->GetReplicationOwner() && CachedReplicator->GetReplicationOwner()->PlayerInput)
	{
		TSet<FName> RemovedMasks = UsedBindings.Difference(NewBindings);
		TSet<FName> AddedMasks = NewBindings.Difference(UsedBindings);

		UPlayerInput* Input = CachedReplicator->GetReplicationOwner()->PlayerInput;
		for (int32 Idx = 0; Idx < Input->DebugExecBindings.Num(); Idx++)
		{
			FKeyBind& DebugBinding = Input->DebugExecBindings[Idx];
			const bool bRemoveMask = RemovedMasks.Contains(DebugBinding.Key.GetFName());
			const bool bAddMask = AddedMasks.Contains(DebugBinding.Key.GetFName());

			if (bAddMask || bRemoveMask)
			{
				DebugBinding.bDisabled = bAddMask;
			}
		}

		UsedBindings = NewBindings;
	}
}

bool UGameplayDebuggerLocalController::IsKeyBound(const FName KeyName) const
{
	return UsedBindings.Contains(KeyName);
}

void UGameplayDebuggerLocalController::OnActivationPressed()
{
	if (CachedReplicator)
	{
		const double HoldTimeThr = 0.2 * (FApp::UseFixedTimeStep() ? (FApp::GetFixedDeltaTime() * 60.) : 1.);

		CachedReplicator->GetWorldTimerManager().SetTimer(StartSelectingActorHandle, this, &UGameplayDebuggerLocalController::OnStartSelectingActor, static_cast<float>(HoldTimeThr));
	}
}

void UGameplayDebuggerLocalController::OnActivationPressedWithModifier()
{
	if (CachedReplicator)
	{
		//OnStartSelectingLocalPlayer();
		const double HoldTimeThr = 0.2 * (FApp::UseFixedTimeStep() ? (FApp::GetFixedDeltaTime() * 60.) : 1.);
		
		CachedReplicator->GetWorldTimerManager().SetTimer(StartSelectingActorHandle, this, &UGameplayDebuggerLocalController::OnStartSelectingLocalPlayer, static_cast<float>(HoldTimeThr));
	}
}

void UGameplayDebuggerLocalController::OnActivationReleased()
{
	ToggleActivation(ESelectionMode::BestPawnCandidate);
}

void UGameplayDebuggerLocalController::OnActivationReleasedWithModifier()
{
	ToggleActivation(ESelectionMode::LocalPlayer);
}

void UGameplayDebuggerLocalController::ToggleActivation(const ESelectionMode SelectionMode)
{
	if (CachedReplicator)
	{
		const UWorld* World = CachedReplicator->GetWorld();
		if (!bIsSelectingActor || StartSelectingActorHandle.IsValid())
		{
			bPrevLocallyEnabled = bIsLocallyEnabled;
			bIsLocallyEnabled = !CachedReplicator->IsEnabled();
			CachedReplicator->SetEnabled(bIsLocallyEnabled);

			if (bIsLocallyEnabled)
			{
				bPrevScreenMessagesEnabled = GAreScreenMessagesEnabled;
				GAreScreenMessagesEnabled = false;
				DebugActorCandidate = nullptr;

				if (SelectionMode == ESelectionMode::BestPawnCandidate)
				{
					OnSelectActorTick();
				}

				// If no actor got selected use local player
				if (DebugActorCandidate == nullptr)
				{
					OnSelectLocalPlayer();
				}
			}
			else
			{
				// if Screen message are still disabled, restore previous state
				if (!GAreScreenMessagesEnabled)
				{
					GAreScreenMessagesEnabled = bPrevScreenMessagesEnabled;
				}
			}
		}

		World->GetTimerManager().ClearTimer(StartSelectingActorHandle);
		World->GetTimerManager().ClearTimer(SelectActorTickHandle);

		CachedReplicator->MarkComponentsRenderStateDirty();
	}

	StartSelectingActorHandle.Invalidate();
	SelectActorTickHandle.Invalidate();
	bIsSelectingActor = false;

	if (CachedReplicator && (bPrevLocallyEnabled != bIsLocallyEnabled))
	{
		CachedPlayerManager->RefreshInputBindings(*CachedReplicator);
	}
}

void UGameplayDebuggerLocalController::OnCategory0Pressed()
{
	ToggleSlotState((ActiveRowIdx * NumCategoriesPerRow) + 0);
}

void UGameplayDebuggerLocalController::OnCategory1Pressed()
{
	ToggleSlotState((ActiveRowIdx * NumCategoriesPerRow) + 1);
}

void UGameplayDebuggerLocalController::OnCategory2Pressed()
{
	ToggleSlotState((ActiveRowIdx * NumCategoriesPerRow) + 2);
}

void UGameplayDebuggerLocalController::OnCategory3Pressed()
{
	ToggleSlotState((ActiveRowIdx * NumCategoriesPerRow) + 3);
}

void UGameplayDebuggerLocalController::OnCategory4Pressed()
{
	ToggleSlotState((ActiveRowIdx * NumCategoriesPerRow) + 4);
}

void UGameplayDebuggerLocalController::OnCategory5Pressed()
{
	ToggleSlotState((ActiveRowIdx * NumCategoriesPerRow) + 5);
}

void UGameplayDebuggerLocalController::OnCategory6Pressed()
{
	ToggleSlotState((ActiveRowIdx * NumCategoriesPerRow) + 6);
}

void UGameplayDebuggerLocalController::OnCategory7Pressed()
{
	ToggleSlotState((ActiveRowIdx * NumCategoriesPerRow) + 7);
}

void UGameplayDebuggerLocalController::OnCategory8Pressed()
{
	ToggleSlotState((ActiveRowIdx * NumCategoriesPerRow) + 8);
}

void UGameplayDebuggerLocalController::OnCategory9Pressed()
{
	ToggleSlotState((ActiveRowIdx * NumCategoriesPerRow) + 9);
}

void UGameplayDebuggerLocalController::OnCategoryRowUpPressed()
{
	const int32 NumRows = (NumCategorySlots + (NumCategoriesPerRow-1)) / NumCategoriesPerRow;
	ActiveRowIdx = (NumRows > 1) ? ((ActiveRowIdx + NumRows - 1) % NumRows) : 0;
}

void UGameplayDebuggerLocalController::OnCategoryRowDownPressed()
{
	const int32 NumRows = (NumCategorySlots + (NumCategoriesPerRow-1)) / NumCategoriesPerRow;
	ActiveRowIdx = (NumRows > 1) ? ((ActiveRowIdx + 1) % NumRows) : 0;
}

void UGameplayDebuggerLocalController::OnCategoryBindingEvent(int32 CategoryId, int32 HandlerId)
{
	if (CachedReplicator)
	{
		CachedReplicator->SendCategoryInputEvent(CategoryId, HandlerId);
	}
}

void UGameplayDebuggerLocalController::OnExtensionBindingEvent(int32 ExtensionId, int32 HandlerId)
{
	if (CachedReplicator)
	{
		CachedReplicator->SendExtensionInputEvent(ExtensionId, HandlerId);
	}
}

void UGameplayDebuggerLocalController::OnStartSelectingActor()
{
	OnStartSelecting(ESelectionMode::BestPawnCandidate);
}

void UGameplayDebuggerLocalController::OnStartSelectingLocalPlayer()
{
	OnStartSelecting(ESelectionMode::LocalPlayer);
}

void UGameplayDebuggerLocalController::OnStartSelecting(ESelectionMode SelectionMode)
{
	StartSelectingActorHandle.Invalidate();
	if (CachedReplicator)
	{
		if (!CachedReplicator->IsEnabled())
		{
			bPrevLocallyEnabled = bIsLocallyEnabled;
			bIsLocallyEnabled = true;
			CachedReplicator->SetEnabled(bIsLocallyEnabled);
			bPrevScreenMessagesEnabled = GAreScreenMessagesEnabled;
			GAreScreenMessagesEnabled = false;
		}

		bIsSelectingActor = true;
		DebugActorCandidate = nullptr;

		if (SelectionMode == ESelectionMode::BestPawnCandidate)
		{
			const bool bLooping = true;
			CachedReplicator->GetWorldTimerManager().SetTimer(SelectActorTickHandle, this, &UGameplayDebuggerLocalController::OnSelectActorTick, 0.01f, bLooping);

			OnSelectActorTick();
		}
		else if (SelectionMode == ESelectionMode::LocalPlayer)
		{
			OnSelectLocalPlayer();
		}
	}
}

void UGameplayDebuggerLocalController::OnSelectActorTick()
{
	APlayerController* OwnerPC = CachedReplicator ? CachedReplicator->GetReplicationOwner().Get() : nullptr;
	if (OwnerPC)
	{
		FVector ViewLocation = FVector::ZeroVector;
		FVector ViewDirection = FVector::ForwardVector;
		if (!CachedReplicator->GetViewPoint(ViewLocation, ViewDirection))
		{
			AGameplayDebuggerPlayerManager::GetViewPoint(*OwnerPC, ViewLocation, ViewDirection);
		}

		const UGameplayDebuggerUserSettings* Settings = GetDefault<UGameplayDebuggerUserSettings>();
		const FVector::FReal MaxScanDistance = Settings->MaxViewDistance;
		const FVector::FReal MinViewDirDot = FMath::Cos(FMath::DegreesToRadians(Settings->MaxViewAngle));

		AActor* BestCandidate = nullptr;
		FVector::FReal BestScore = MinViewDirDot;

		for (APawn* TestPawn : TActorRange<APawn>(OwnerPC->GetWorld()))
		{
			if (!TestPawn->IsHidden() && TestPawn->GetActorEnableCollision() &&
				!TestPawn->IsA(ASpectatorPawn::StaticClass()) &&
				TestPawn != OwnerPC->GetPawn())
			{
				FVector DirToPawn = (TestPawn->GetActorLocation() - ViewLocation);
				FVector::FReal DistToPawn = DirToPawn.Size();
				if (FMath::IsNearlyZero(DistToPawn))
				{
					DirToPawn = ViewDirection;
					DistToPawn = 1.;
				}
				else
				{
					DirToPawn /= DistToPawn;
				}

				const FVector::FReal ViewDot = FVector::DotProduct(ViewDirection, DirToPawn);
				if (DistToPawn < MaxScanDistance && ViewDot > BestScore)
				{
					BestScore = ViewDot;
					BestCandidate = TestPawn;
				}
			}
		}

		// cache to avoid multiple RPC with the same actor
		if (DebugActorCandidate != BestCandidate)
		{
			DebugActorCandidate = BestCandidate;
			CachedReplicator->SetDebugActor(BestCandidate, true);
		}
	}
}

void UGameplayDebuggerLocalController::ToggleSlotState(int32 SlotIdx)
{
	if (CachedReplicator && SlotCategoryIds.IsValidIndex(SlotIdx) && SlotCategoryIds[SlotIdx].Num())
	{
		const bool bIsEnabled = CachedReplicator->IsCategoryEnabled(SlotCategoryIds[SlotIdx][0]);
		for (int32 Idx = 0; Idx < SlotCategoryIds[SlotIdx].Num(); Idx++)
		{
			// Inform server of category's new enabled state
			const bool bNewEnabled = !bIsEnabled;
			const int32 CategoryId = SlotCategoryIds[SlotIdx][Idx];
			CachedReplicator->SetCategoryEnabled(CategoryId, bNewEnabled);

			// Save new enabled state to disk
			const FName CategoryName = CachedReplicator->GetCategory(CategoryId)->GetCategoryName();
			UGameplayDebuggerUserSettings* UserSettings = GetMutableDefault<UGameplayDebuggerUserSettings>();
			if (bNewEnabled)
			{
				UserSettings->EnabledCategories.Add(CategoryName);
			}
			else
			{
				UserSettings->EnabledCategories.Remove(CategoryName);
			}
			UserSettings->SaveConfig();
		}
		// removed call to MarkComponentsRenderStateDirty since CachedReplicator->SetCategoryEnabled
		// calls it already 
	}
}

FString UGameplayDebuggerLocalController::GetKeyDescriptionShort(const FKey& KeyBind) const
{
	return FString::Printf(TEXT("[%s]"), *KeyBind.GetFName().ToString());
}

FString UGameplayDebuggerLocalController::GetKeyDescriptionLong(const FKey& KeyBind) const
{
	const FString KeyDisplay = KeyBind.GetDisplayName().ToString();
	const FString KeyName = KeyBind.GetFName().ToString();
	return (KeyDisplay == KeyName) ? FString::Printf(TEXT("[%s]"), *KeyDisplay) : FString::Printf(TEXT("%s [%s key])"), *KeyDisplay, *KeyName);
}

#if WITH_EDITOR
void UGameplayDebuggerLocalController::OnSelectionChanged(UObject* Object)
{
	USelection* Selection = Cast<USelection>(Object);
	if (Selection && CachedReplicator)
	{
		AActor* SelectedActor = nullptr;
		for (int32 Idx = 0; Idx < Selection->Num(); Idx++)
		{
			SelectedActor = Cast<AActor>(Selection->GetSelectedObject(Idx));
			if (SelectedActor)
			{
				break;
			}
		}

		if (SelectedActor)
		{
			// Since this is used from a delegate we need to filter out objects not belonging to the same pie instance/world
			if (SelectedActor && SelectedActor->GetWorld() != GetWorld())	
			{
				return;
			}

			CachedReplicator->SetDebugActor(SelectedActor, false);
			CachedReplicator->CollectCategoryData(/*bForce=*/true);
		}
	}
}

void UGameplayDebuggerLocalController::OnSelectedObject(UObject* Object)
{
	// Since this is used from a delegate we need to filter out objects not belonging to the same pie instance/world
	if (Object && Object->GetWorld() != GetWorld())	
	{
		return;
	}

	AController* SelectedController = Cast<AController>(Object);
	APawn* SelectedPawn = SelectedController ? SelectedController->GetPawn().Get() : Cast<APawn>(Object);
	if (CachedReplicator && SelectedPawn && SelectedPawn->IsSelected())
	{
		CachedReplicator->SetDebugActor(SelectedPawn, false);
		CachedReplicator->CollectCategoryData(/*bForce=*/true);
	}
}
#endif // WITH_EDITOR

void UGameplayDebuggerLocalController::RebuildDataPackMap()
{
	DataPackMap.SetNum(NumCategories);
	
	// category: get all categories from slot and combine data pack data if category header is not displayed
	for (int32 SlotIdx = 0; SlotIdx < NumCategorySlots; SlotIdx++)
	{
		TArray<int32> NoHeaderCategories;
		int32 FirstVisibleCategoryId = INDEX_NONE;

		for (int32 InnerIdx = 0; InnerIdx < SlotCategoryIds[SlotIdx].Num(); InnerIdx++)
		{
			const int32 CategoryId = SlotCategoryIds[SlotIdx][InnerIdx];
			
			TSharedRef<FGameplayDebuggerCategory> Category = CachedReplicator->GetCategory(CategoryId);
			if (!Category->IsCategoryHeaderVisible())
			{
				NoHeaderCategories.Add(CategoryId);
			}
			else
			{
				DataPackMap[CategoryId].Add(CategoryId);
				
				if (FirstVisibleCategoryId == INDEX_NONE)
				{
					FirstVisibleCategoryId = CategoryId;
				}
			}
		}

		if ((FirstVisibleCategoryId != INDEX_NONE) && NoHeaderCategories.Num())
		{
			DataPackMap[FirstVisibleCategoryId].Append(NoHeaderCategories);
		}
	}
}

#if WITH_EDITOR
void UGameplayDebuggerLocalController::OnBeginPIE(const bool bIsSimulating)
{
	bActivateOnPIEEnd = bIsLocallyEnabled;
	if (bIsLocallyEnabled)
	{
		ToggleActivation();
	}
}

void UGameplayDebuggerLocalController::OnEndPIE(const bool bIsSimulating)
{
	if (bActivateOnPIEEnd && !bIsLocallyEnabled)
	{
		ToggleActivation();
	}
}
#endif // WITH_EDITOR

//----------------------------------------------------------------------//
// FGameplayDebuggerConsoleCommands 
//----------------------------------------------------------------------//
/**  Helper structure to declare/define console commands in the source file and to access UGameplayDebuggerLocalController protected members */
struct FGameplayDebuggerConsoleCommands
{
private:
	static UGameplayDebuggerLocalController* GetController(UWorld* InWorld)
	{
		if (!UGameplayDebuggerLocalController::bConsoleCommandsEnabled)
		{
			return nullptr;
		}

		UGameplayDebuggerLocalController* Controller = nullptr;

		APlayerController* LocalPC = GEngine->GetFirstLocalPlayerController(InWorld);
		if (LocalPC)
		{
			Controller = AGameplayDebuggerPlayerManager::GetCurrent(InWorld).GetLocalController(*LocalPC);
		}
#if WITH_EDITOR
		else if (InWorld != nullptr && InWorld->IsGameWorld() == false)
		{
			Controller = AGameplayDebuggerPlayerManager::GetCurrent(InWorld).GetEditorController();
		}
#endif // WITH_EDITOR

		UE_CLOGF(Controller == nullptr, LogConsoleResponse, Error, "GameplayDebugger not available");
		return Controller;
	}

	static void EnableGameplayDebugger(const TArray<FString>& Args, UWorld* InWorld)
	{
		if (UGameplayDebuggerLocalController* Controller = GetController(InWorld))
		{
			bool bEnable = true;
			if (Args.Num() > 0)
			{
				LexFromString(bEnable, *Args[0]);
			}

			if (Controller->bIsLocallyEnabled != bEnable)
			{
				Controller->ToggleActivation();	
			}
		}
	}

	static void ToggleGameplayDebugger(UWorld* InWorld)
	{
		if (UGameplayDebuggerLocalController* Controller = GetController(InWorld))
		{
			Controller->ToggleActivation();
		}
	}

	static void SelectLocalPlayer(UWorld* InWorld)
	{
		if (UGameplayDebuggerLocalController* Controller = GetController(InWorld))
		{
			Controller->OnSelectLocalPlayer();
		}
	}

	static void SelectPreviousRow(UWorld* InWorld)
	{
		if (UGameplayDebuggerLocalController* Controller = GetController(InWorld))
		{
			Controller->OnCategoryRowUpPressed();
		}
	}

	static void SelectNextRow(UWorld* InWorld)
	{
		if (UGameplayDebuggerLocalController* Controller = GetController(InWorld))
		{
			Controller->OnCategoryRowDownPressed();
		}
	}

	static void ToggleCategory(const TArray<FString>& Args, UWorld* InWorld)
	{
		UGameplayDebuggerLocalController* Controller = GetController(InWorld);
		if (Controller == nullptr)
		{
			return;
		}

		if (Args.Num() != 1)
		{
			UE_LOGF(LogConsoleResponse, Error, "Missing category index parameter. Usage: gdt.ToggleCategory <CategoryIdx>");
			return;
		}

		if (!Args[0].IsNumeric())
		{
			UE_LOGF(LogConsoleResponse, Error, "Must provide numerical value as index. Usage: gdt.ToggleCategory <CategoryIdx>");
			return;
		}
		
		const int32 SlotIdx = TCString<TCHAR>::Atoi(*Args[0]);
		const int32 NumSlots = Controller->SlotCategoryIds.Num();
		const int32 NumSlotsPerRow = UGameplayDebuggerLocalController::NumCategoriesPerRow;
		const int32 NumRows = (NumSlots + (NumSlotsPerRow-1)) / NumSlotsPerRow;

		const bool bIsLastRowActive = (Controller->ActiveRowIdx == NumRows-1);
		const int32 NumSlotsOnActiveRow = bIsLastRowActive ? NumSlots - (NumSlotsPerRow * (NumRows-1)) : NumSlotsPerRow;
		const int32 MaxSlotIdx = FMath::Max(0, FMath::Min(NumSlots, NumSlotsOnActiveRow)-1);
		
		if (!(Controller->SlotCategoryIds.IsValidIndex(SlotIdx) && SlotIdx <= MaxSlotIdx))
		{
			UE_LOGF(LogConsoleResponse, Error, "Requires a category index in the active row [0..%d]. Usage: gdt.ToggleCategory CategoryIndex", MaxSlotIdx);
			return;
		}
		
		Controller->ToggleSlotState((Controller->ActiveRowIdx * NumSlotsPerRow)+SlotIdx);
	}

	static void EnableCategoryName(const TArray<FString>& Args, UWorld* InWorld)
	{
		UGameplayDebuggerLocalController* Controller = GetController(InWorld);
		if (Controller == nullptr || Controller->CachedReplicator == nullptr)
		{
			UE_CLOGF(Controller != nullptr && Controller->CachedReplicator == nullptr, 
				LogConsoleResponse, Error, "Failed due to CachedReplicator being Null");
			return;
		}

		if (Args.Num() < 1)
		{
			UE_LOGF(LogConsoleResponse, Error, "Missing category name parameter. Usage: gdt.EnableCategory <CategoryNamePattern> [Enable]");
			return;
		}

		bool bEnable = (Args.Num() == 1);
		if (Args.Num() > 1)
		{
			LexFromString(bEnable, *Args[1]);
		}

		for (int32 CategoryIndex = 0; CategoryIndex < Controller->CachedReplicator->GetNumCategories(); ++CategoryIndex)
		{
			const TSharedRef<FGameplayDebuggerCategory> Category = Controller->CachedReplicator->GetCategory(CategoryIndex);
			const FString CategoryName = Category->GetCategoryName().ToString();
			if (CategoryName.Find(Args[0], ESearchCase::IgnoreCase) != INDEX_NONE)
			{
				Controller->CachedReplicator->SetCategoryEnabled(CategoryIndex, bEnable);
			}
		}
	}

	static void SetFontSize(const TArray<FString>& Args, UWorld* InWorld)
	{
		if (Args.Num() != 1)
		{
			UE_LOGF(LogConsoleResponse, Error, "Missing \'fontSize\' parameter. Usage: gdt.fontsize <fontSize>");
			return;
		}

		if (!Args[0].IsNumeric())
		{
			UE_LOGF(LogConsoleResponse, Error, "Must provide numerical value as \'fontSize\'. Usage: gdt.fontsize <fontSize>");
			return;
		}
	
		if (const UGameplayDebuggerLocalController* LocalController = GetController(InWorld))
		{
			UGameplayDebuggerUserSettings::SetFontSize(TCString<TCHAR>::Atoi(*Args[0]));
			check(LocalController->HUDFont);
			LocalController->HUDFont->LegacyFontSize = UGameplayDebuggerUserSettings::GetFontSize();
		}
	}

	/** For legacy command: EnableGDT */
	static FAutoConsoleCommandWithWorld LegacyEnableDebuggerCmd;

	/** Various gameplay debugger commands: gdt.<command> */
	static FAutoConsoleCommandWithWorldAndArgs EnableDebuggerCmd;
	static FAutoConsoleCommandWithWorld ToggleDebuggerCmd;
	static FAutoConsoleCommandWithWorld SelectLocalPlayerCmd;
	static FAutoConsoleCommandWithWorld SelectPreviousRowCmd;
	static FAutoConsoleCommandWithWorld SelectNextRowCmd;
	static FAutoConsoleCommandWithWorldAndArgs ToggleCategoryCmd;
	static FAutoConsoleCommandWithWorldAndArgs EnableCategoryNameCmd;
	static FAutoConsoleCommandWithWorldAndArgs SetFontSizeCmd;
};

FAutoConsoleCommandWithWorld FGameplayDebuggerConsoleCommands::LegacyEnableDebuggerCmd(
	TEXT("EnableGDT"),
	TEXT("Toggles Gameplay Debugger Tool"),
	FConsoleCommandWithWorldDelegate::CreateStatic(&FGameplayDebuggerConsoleCommands::ToggleGameplayDebugger)
);

FAutoConsoleCommandWithWorldAndArgs FGameplayDebuggerConsoleCommands::EnableDebuggerCmd(
	TEXT("gdt.Enable"),
	TEXT("Enable Gameplay Debugger Tool"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(&FGameplayDebuggerConsoleCommands::EnableGameplayDebugger)
);

FAutoConsoleCommandWithWorld FGameplayDebuggerConsoleCommands::ToggleDebuggerCmd(
	TEXT("gdt.Toggle"),
	TEXT("Toggles Gameplay Debugger Tool"),
	FConsoleCommandWithWorldDelegate::CreateStatic(&FGameplayDebuggerConsoleCommands::ToggleGameplayDebugger)
);

FAutoConsoleCommandWithWorld FGameplayDebuggerConsoleCommands::SelectLocalPlayerCmd(
	TEXT("gdt.SelectLocalPlayer"),
	TEXT("Selects the local player for debugging"),
	FConsoleCommandWithWorldDelegate::CreateStatic(FGameplayDebuggerConsoleCommands::SelectLocalPlayer)
);

FAutoConsoleCommandWithWorld FGameplayDebuggerConsoleCommands::SelectPreviousRowCmd(
	TEXT("gdt.SelectPreviousRow"),
	TEXT("Selects previous row"),
	FConsoleCommandWithWorldDelegate::CreateStatic(FGameplayDebuggerConsoleCommands::SelectPreviousRow)
);

FAutoConsoleCommandWithWorld FGameplayDebuggerConsoleCommands::SelectNextRowCmd(
	TEXT("gdt.SelectNextRow"),
	TEXT("Selects next row"),
	FConsoleCommandWithWorldDelegate::CreateStatic(FGameplayDebuggerConsoleCommands::SelectNextRow)
);

FAutoConsoleCommandWithWorldAndArgs FGameplayDebuggerConsoleCommands::ToggleCategoryCmd(
	TEXT("gdt.ToggleCategory"),
	TEXT("Toggles specific category index"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(&FGameplayDebuggerConsoleCommands::ToggleCategory)
);

FAutoConsoleCommandWithWorldAndArgs FGameplayDebuggerConsoleCommands::EnableCategoryNameCmd(
	TEXT("gdt.EnableCategoryName"),
	TEXT("Enables/disables categories matching given substring. Use: gdt.EnableCategoryName <CategoryNamePart> [Enable]"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(&FGameplayDebuggerConsoleCommands::EnableCategoryName)
);

FAutoConsoleCommandWithWorldAndArgs FGameplayDebuggerConsoleCommands::SetFontSizeCmd(
	TEXT("gdt.fontsize"),
	TEXT("Configures gameplay debugger's font size. Usage: gdt.fontsize <fontSize> (default = 10)"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(&FGameplayDebuggerConsoleCommands::SetFontSize)
);

#endif // WITH_GAMEPLAY_DEBUGGER_MENU