// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayDebuggerModule.h"

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Engine/World.h"
#include "GameplayDebugger.h"
#include "ISettingsModule.h"
#include "GameplayDebuggerAddonManager.h"
#include "GameplayDebuggerPlayerManager.h"
#include "GameplayDebuggerConfig.h"

#include "GameplayDebuggerExtension_Spectator.h"
#include "GameplayDebuggerExtension_HUD.h"

#include "Iris/IrisConfig.h"


static bool bAutoCreateGameplayDebuggerManager = true;
static FAutoConsoleVariableRef CVarAutoCreateGameplayDebuggerManager(
	TEXT("GameplayDebugger.AutoCreateGameplayDebuggerManager"),
	bAutoCreateGameplayDebuggerManager,
	TEXT("When false it stops auto-spawning the Gameplay Debugger Manager.")
);

FOnLocalControllerInitialized FGameplayDebuggerModule::OnLocalControllerInitialized;
FOnLocalControllerUninitialized FGameplayDebuggerModule::OnLocalControllerUninitialized;

#if WITH_EDITOR
FOnDebuggerEdMode FGameplayDebuggerModule::OnDebuggerEdModeActivation;
#endif //WITH_EDITOR

IMPLEMENT_MODULE(FGameplayDebuggerModule, GameplayDebugger)

void FGameplayDebuggerModule::StartupModule()
{
	// This code will execute after your module is loaded into memory (but after global variables are initialized, of course.)
	FWorldDelegates::OnPostWorldInitialization.AddRaw(this, &FGameplayDebuggerModule::OnWorldInitialized);

	UGameplayDebuggerConfig* SettingsCDO = UGameplayDebuggerConfig::StaticClass()->GetDefaultObject<UGameplayDebuggerConfig>();
	if (SettingsCDO)
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
		if (SettingsModule)
		{
			SettingsModule->RegisterSettings("Project", "Engine", "GameplayDebugger",
				NSLOCTEXT("GameplayDebuggerModule", "SettingsName", "Gameplay Debugger"),
				NSLOCTEXT("GameplayDebuggerModule", "SettingsDescription", "Settings for the gameplay debugger tool."),
				SettingsCDO);
		}

		AddonManager.RegisterExtension("GameHUD", FOnGetExtension::CreateStatic(&FGameplayDebuggerExtension_HUD::MakeInstance));
		AddonManager.RegisterExtension("Spectator", FOnGetExtension::CreateStatic(&FGameplayDebuggerExtension_Spectator::MakeInstance));
		AddonManager.NotifyExtensionsChanged();
	}
}

void FGameplayDebuggerModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
	FWorldDelegates::OnPostWorldInitialization.RemoveAll(this);

	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	if (SettingsModule)
	{
		SettingsModule->UnregisterSettings("Project", "Engine", "GameplayDebugger");
	}
}

void FGameplayDebuggerModule::RegisterCategory(FName CategoryName, IGameplayDebugger::FOnGetCategory MakeInstanceDelegate, EGameplayDebuggerCategoryState CategoryState, int32 SlotIdx)
{
	AddonManager.RegisterCategory(CategoryName, MakeInstanceDelegate, CategoryState, SlotIdx);
}

void FGameplayDebuggerModule::UnregisterCategory(FName CategoryName)
{
	AddonManager.UnregisterCategory(CategoryName);
}

void FGameplayDebuggerModule::NotifyCategoriesChanged()
{
	AddonManager.NotifyCategoriesChanged();
}

void FGameplayDebuggerModule::RegisterExtension(FName ExtensionName, IGameplayDebugger::FOnGetExtension MakeInstanceDelegate)
{
	AddonManager.RegisterExtension(ExtensionName, MakeInstanceDelegate);
}

void FGameplayDebuggerModule::UnregisterExtension(FName ExtensionName)
{
	AddonManager.UnregisterExtension(ExtensionName);
}

void FGameplayDebuggerModule::NotifyExtensionsChanged()
{
	AddonManager.NotifyExtensionsChanged();
}

FGameplayDebuggerAddonManager& FGameplayDebuggerAddonManager::GetCurrent()
{
	FGameplayDebuggerModule& Module = FModuleManager::LoadModuleChecked<FGameplayDebuggerModule>("GameplayDebugger");
	return Module.AddonManager;
}

AGameplayDebuggerPlayerManager& AGameplayDebuggerPlayerManager::GetCurrent(UWorld* World)
{
	FGameplayDebuggerModule& Module = FModuleManager::LoadModuleChecked<FGameplayDebuggerModule>("GameplayDebugger");
	return Module.GetPlayerManager(World);
}

AGameplayDebuggerPlayerManager& FGameplayDebuggerModule::GetPlayerManager(UWorld* World)
{
	const int32 PurgeInvalidWorldsSize = 5;
	if (PlayerManagers.Num() > PurgeInvalidWorldsSize)
	{
		for (TMap<TWeakObjectPtr<UWorld>, TWeakObjectPtr<AGameplayDebuggerPlayerManager> >::TIterator It(PlayerManagers); It; ++It)
		{
			if (!It.Key().IsValid())
			{
				It.RemoveCurrent();
			}
			else if (!It.Value().IsValid())
			{
				It.RemoveCurrent();
			}
		}
	}

	TWeakObjectPtr<AGameplayDebuggerPlayerManager> Manager = PlayerManagers.FindRef(World);
	AGameplayDebuggerPlayerManager* ManagerOb = Manager.Get();

	if (ManagerOb == nullptr)
	{
		ManagerOb = World->SpawnActor<AGameplayDebuggerPlayerManager>();
		PlayerManagers.Add(World, ManagerOb);
		ManagerOb->Init();
	}

	check(ManagerOb);
	return *ManagerOb;
}

void FGameplayDebuggerModule::OnWorldInitialized(UWorld* World, const UWorld::InitializationValues IVS)
{
	// make sure that world has valid player manager, create when it doesn't
	if (World && bAutoCreateGameplayDebuggerManager && (World->IsGameWorld() || World->IsEditorWorld()))
	{
		GetPlayerManager(World);
	}
}