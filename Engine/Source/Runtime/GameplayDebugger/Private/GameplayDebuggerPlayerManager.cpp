// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayDebuggerPlayerManager.h"
#include "Engine/World.h"
#include "Components/InputComponent.h"
#include "GameplayDebuggerCategoryReplicator.h"
#include "GameplayDebuggerLocalController.h"
#include "Engine/DebugCameraController.h"
#include "Engine/InputDelegateBinding.h"
#include "GameplayDebuggerConfig.h"
#include "UnrealEngine.h"
#include "Engine/LocalPlayer.h"
#include "Net/UnrealNetwork.h"
#include "GameFramework/GameModeBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayDebuggerPlayerManager)

AGameplayDebuggerPlayerManager::AGameplayDebuggerPlayerManager(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.bAllowTickOnDedicatedServer = true;
	PrimaryActorTick.bTickEvenWhenPaused = true;
	PrimaryActorTick.bStartWithTickEnabled = false;
	PrimaryActorTick.TickInterval = 0.5f;

#if WITH_EDITOR
	SetIsTemporarilyHiddenInEditor(true);
#endif

#if WITH_EDITORONLY_DATA
	bHiddenEdLevel = true;
	bHiddenEdLayer = true;
	bHiddenEd = true;
	bEditable = false;
#endif

	bIsLocal = false;
	bInitialized = false;
	bEditorTimeTick = false;
}

void AGameplayDebuggerPlayerManager::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		if (InputComponent == nullptr)
		{
			// create an InputComponent object so that the level script actor can bind key events
			InputComponent = NewObject<UInputComponent>(this, TEXT("GameplayDebug_Input"));
			InputComponent->RegisterComponent();
		}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (UInputDelegateBinding::SupportsInputDelegate(GetClass()))
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			UInputDelegateBinding::BindInputDelegates(GetClass(), InputComponent);
		}
#if WITH_EDITORONLY_DATA 
		const UWorld* World = GetWorld();
		bEditorTimeTick = (World != nullptr) && (World->IsEditorWorld() == true) && (World->IsGameWorld() == false);
#endif // WITH_EDITORONLY_DATA 
	}
}

void AGameplayDebuggerPlayerManager::BeginPlay()
{
	Super::BeginPlay();

	UWorld* World = GetWorld();
	check(World);
	const ENetMode NetMode = World->GetNetMode();
	
	bHasAuthority = FGameplayDebuggerUtils::IsAuthority(World);
	bIsLocal = (NetMode != NM_DedicatedServer);
	bInitialized = true;

	if (bHasAuthority)
	{
		UpdateAuthReplicators();
		SetActorTickEnabled(true);
	}
	
	for (int32 Idx = 0; Idx < PendingRegistrations.Num(); Idx++)
	{
		RegisterReplicator(*PendingRegistrations[Idx]);
	}

	PendingRegistrations.Empty();

	FNetworkReplayDelegates::OnScrubTeardown.AddUObject(this, &ThisClass::OnReplayScrubTeardown);
	FGameModeEvents::GameModeLogoutEvent.AddUObject(this, &ThisClass::OnGameModeLogout);
}

void AGameplayDebuggerPlayerManager::BeginDestroy()
{
	Super::BeginDestroy();

	// Make sure the tick is completely disabled
	SetTickableTickType(ETickableTickType::Never);
}

void AGameplayDebuggerPlayerManager::EndPlay(const EEndPlayReason::Type Reason)
{
	Super::EndPlay(Reason);

	for (int32 Idx = 0; Idx < PlayerData.Num(); Idx++)
	{
		FGameplayDebuggerPlayerData& TestData = PlayerData[Idx];
		if (IsValid(TestData.Controller))
		{
			TestData.Controller->Cleanup();
			TestData.Controller = nullptr;
		}
	}

	FNetworkReplayDelegates::OnScrubTeardown.RemoveAll(this);
	FGameModeEvents::GameModeLogoutEvent.RemoveAll(this);
}

void AGameplayDebuggerPlayerManager::Init()
{
#if WITH_EDITOR
	UWorld* World = GetWorld();
	if (World != nullptr && World->WorldType == EWorldType::Editor && (GetDefault<UGameplayDebuggerUserSettings>()->bEnableGameplayDebuggerInEditor))
	{
		bHasAuthority = true;
		bIsLocal = true;
		bInitialized = true;

		AGameplayDebuggerCategoryReplicator* Replicator = World->SpawnActorDeferred<AGameplayDebuggerCategoryReplicator>(AGameplayDebuggerCategoryReplicator::StaticClass(), FTransform::Identity);
		Replicator->SetReplicatorOwner(nullptr);
		Replicator->FinishSpawning(FTransform::Identity, true);
		SetActorTickEnabled(true);

		EditorWorldData.Replicator = Replicator;

		Replicator->InitForEditor();

		EditorWorldData.Controller = NewObject<UGameplayDebuggerLocalController>(this, TEXT("GameplayDebug_Controller_Editor"));
		EditorWorldData.Controller->Initialize(*Replicator, *this);
		
	}
#endif // WITH_EDITOR
}

void AGameplayDebuggerPlayerManager::TickActor(float DeltaTime, enum ELevelTick TickType, FActorTickFunction& ThisTickFunction)
{
	Super::TickActor(DeltaTime, TickType, ThisTickFunction);
	UpdateAuthReplicators();
};

void AGameplayDebuggerPlayerManager::UpdateAuthReplicators()
{
	UWorld* World = GetWorld();
	for (int32 Idx = PlayerData.Num() - 1; Idx >= 0; Idx--)
	{
		FGameplayDebuggerPlayerData& TestData = PlayerData[Idx];

		if (IsValid(TestData.Replicator) && TestData.Replicator->GetReplicationOwner().GetResidence() == EResidence::Remote)
		{
			continue;
		}

		if (!IsValid(TestData.Replicator) || !IsValid(TestData.Replicator->GetReplicationOwner()))
		{
			if (IsValid(TestData.Replicator))
			{
				World->DestroyActor(TestData.Replicator);
			}

			if (IsValid(TestData.Controller))
			{
				TestData.Controller->Cleanup();
			}

			PlayerData.RemoveAt(Idx, EAllowShrinking::No);
		}
	}

#if WITH_GAMEPLAY_DEBUGGER
	for (FConstPlayerControllerIterator It = World->GetPlayerControllerIterator(); It; It++)
	{
		APlayerController* TestPC = It->Get();
		if (TestPC && !TestPC->IsA<ADebugCameraController>())
		{
			const bool bNeedsReplicator = (GetReplicator(*TestPC) == nullptr);
			if (bNeedsReplicator)
			{
				AGameplayDebuggerCategoryReplicator* Replicator = World->SpawnActorDeferred<AGameplayDebuggerCategoryReplicator>(AGameplayDebuggerCategoryReplicator::StaticClass(), FTransform::Identity);
				Replicator->SetReplicatorOwner(TestPC);
				Replicator->FinishSpawning(FTransform::Identity, true);
			}
		}
	}
#endif // WITH_GAMEPLAY_DEBUGGER

	PrimaryActorTick.TickInterval = PlayerData.Num() ? 5.0f : 0.5f;
}

void AGameplayDebuggerPlayerManager::RegisterReplicator(AGameplayDebuggerCategoryReplicator& Replicator)
{
	if (!bInitialized)
	{
		PendingRegistrations.Add(&Replicator);
		return;
	}

	// keep all player related objects together for easy access and GC
	FGameplayDebuggerPlayerData NewData;
	NewData.Replicator = &Replicator;

#if WITH_GAMEPLAY_DEBUGGER_MENU
	if (bIsLocal)
	{
		APlayerController* OwnerPC = Replicator.GetReplicationOwner();

		NewData.InputComponent = OwnerPC ? NewObject<UInputComponent>(OwnerPC, TEXT("GameplayDebug_Input")) : ToRawPtr(InputComponent);
		check(NewData.InputComponent);
		NewData.InputComponent->Priority = -1;
		NewData.Controller = NewObject<UGameplayDebuggerLocalController>(OwnerPC ? OwnerPC : (AActor*)this, TEXT("GameplayDebug_Controller"));
		NewData.Controller->Initialize(Replicator, *this);
		NewData.Controller->BindInput(*NewData.InputComponent);

		if (OwnerPC)
		{
			OwnerPC->PushInputComponent(NewData.InputComponent);
		}
	}
	else
	{
		NewData.Controller = nullptr;
		NewData.InputComponent = nullptr;
	}
#endif // WITH_GAMEPLAY_DEBUGGER_MENU

	PlayerData.Add(NewData);
}

void AGameplayDebuggerPlayerManager::RefreshInputBindings(AGameplayDebuggerCategoryReplicator& Replicator)
{
#if WITH_GAMEPLAY_DEBUGGER_MENU
	for (int32 Idx = 0; Idx < PlayerData.Num(); Idx++)
	{
		FGameplayDebuggerPlayerData& TestData = PlayerData[Idx];
		if (TestData.Replicator == &Replicator)
		{
			TestData.InputComponent->ClearActionBindings();
			TestData.InputComponent->ClearBindingValues();
			TestData.InputComponent->KeyBindings.Empty();

			TestData.Controller->BindInput(*TestData.InputComponent);
		}
	}
#endif // WITH_GAMEPLAY_DEBUGGER_MENU
}

AGameplayDebuggerCategoryReplicator* AGameplayDebuggerPlayerManager::GetReplicator(const APlayerController& OwnerPC) const
{
	const FGameplayDebuggerPlayerData* DataPtr = GetPlayerData(OwnerPC);
	return DataPtr ? DataPtr->Replicator : nullptr;
}

UInputComponent* AGameplayDebuggerPlayerManager::GetInputComponent(const APlayerController& OwnerPC) const
{
	const FGameplayDebuggerPlayerData* DataPtr = GetPlayerData(OwnerPC);
	return DataPtr ? DataPtr->InputComponent : nullptr;
}

UGameplayDebuggerLocalController* AGameplayDebuggerPlayerManager::GetLocalController(const APlayerController& OwnerPC) const
{
	const FGameplayDebuggerPlayerData* DataPtr = GetPlayerData(OwnerPC);
	return DataPtr ? DataPtr->Controller : nullptr;
}

const FGameplayDebuggerPlayerData* AGameplayDebuggerPlayerManager::GetPlayerData(const APlayerController& OwnerPC) const
{
	for (int32 Idx = 0; Idx < PlayerData.Num(); Idx++)
	{
		const FGameplayDebuggerPlayerData& TestData = PlayerData[Idx];
		if (TestData.Replicator && TestData.Replicator->GetReplicationOwner() == &OwnerPC)
		{
			return &TestData;
		}
	}

	return nullptr;
}

void AGameplayDebuggerPlayerManager::GetViewPoint(const APlayerController& OwnerPC, FVector& OutViewLocation, FVector& OutViewDirection)
{
	UWorld* World = OwnerPC.GetWorld();
	FVector CameraLocation = FVector::ZeroVector;
	FRotator CameraRotation = FRotator::ZeroRotator;
	if (OwnerPC.Player)
	{
		// normal game
		OwnerPC.GetPlayerViewPoint(CameraLocation, CameraRotation);
	}
	else
	{
		// spectator mode
		for (FLocalPlayerIterator It(GEngine, World); It; ++It)
		{
			ADebugCameraController* SpectatorPC = Cast<ADebugCameraController>(It->PlayerController);
			if (SpectatorPC)
			{
				SpectatorPC->GetPlayerViewPoint(CameraLocation, CameraRotation);
				break;
			}
		}
	}

	OutViewLocation = CameraLocation;
	OutViewDirection = CameraRotation.Vector();
}

// FTickableGameObject begin
void AGameplayDebuggerPlayerManager::Tick(float DeltaTime)
{
	Super::Tick(DeltaTime);
	
#if WITH_EDITORONLY_DATA 
	if (EditorWorldData.Replicator)
	{
		FActorTickFunction DummyTickFunction;
		EditorWorldData.Replicator->TickActor(DeltaTime, ELevelTick::LEVELTICK_All, DummyTickFunction);
	}
#endif // WITH_EDITORONLY_DATA 
}

ETickableTickType AGameplayDebuggerPlayerManager::GetTickableTickType() const
{
	return
#if WITH_EDITOR
		IsTickable() ? ETickableTickType::Conditional : 
#endif // WITH_EDITOR
		ETickableTickType::Never;
}

TStatId AGameplayDebuggerPlayerManager::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(AGameplayDebuggerPlayerManager, STATGROUP_Tickables);
}
// FTickableGameObject end

void AGameplayDebuggerPlayerManager::OnGameModeLogout(AGameModeBase* GameMode, AController* Exiting)
{
	if (GameMode && GameMode->GetWorld() == GetWorld())
	{
		UWorld* World = GetWorld();
		for (int32 Idx = PlayerData.Num() - 1; Idx >= 0; Idx--)
		{
			FGameplayDebuggerPlayerData& TestData = PlayerData[Idx];

			if (IsValid(TestData.Replicator) && (TestData.Replicator->GetReplicationOwner() == Exiting))
			{
				if (IsValid(TestData.Replicator))
				{
					World->DestroyActor(TestData.Replicator);
				}

				if (IsValid(TestData.Controller))
				{
					TestData.Controller->Cleanup();
				}

				PlayerData.RemoveAt(Idx, EAllowShrinking::No);
				break;
			}
		}
	}
}

void AGameplayDebuggerPlayerManager::OnReplayScrubTeardown(UWorld* InWorld)
{
	if (GetWorld() == InWorld)
	{
		UpdateAuthReplicators();
	}
}
