// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayDebuggerRenderingComponent.h"
#include "GameplayDebuggerCategoryReplicator.h"
#include "GameplayDebuggerCategory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayDebuggerRenderingComponent)

//////////////////////////////////////////////////////////////////////////
// FGameplayDebuggerCompositeSceneProxy

class FGameplayDebuggerCompositeSceneProxy : public FDebugRenderSceneProxy
{
	friend class FGameplayDebuggerDebugDrawDelegateHelper;
public:
	FGameplayDebuggerCompositeSceneProxy(const UPrimitiveComponent* InComponent) : FDebugRenderSceneProxy(InComponent) { }

	virtual ~FGameplayDebuggerCompositeSceneProxy() override
	{
		for (int32 Idx = 0; Idx < ChildProxies.Num(); Idx++)
		{
			delete ChildProxies[Idx];
		}
	}

	virtual void DrawStaticElements(FStaticPrimitiveDrawInterface* PDI) override
	{
		for (int32 Idx = 0; Idx < ChildProxies.Num(); Idx++)
		{
			ChildProxies[Idx]->DrawStaticElements(PDI);
		}
	}

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
		for (int32 Idx = 0; Idx < ChildProxies.Num(); Idx++)
		{
			ChildProxies[Idx]->GetDynamicMeshElements(Views, ViewFamily, VisibilityMap, Collector);
		}
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result;
		for (int32 Idx = 0; Idx < ChildProxies.Num(); Idx++)
		{
			Result |= ChildProxies[Idx]->GetViewRelevance(View);
		}
		return Result;
	}

	virtual uint32 GetMemoryFootprint(void) const override
	{
		return sizeof(*this) + GetAllocatedSizeInternal();
	}

	void AddChild(FDebugRenderSceneProxy* NewChild)
	{
		ChildProxies.AddUnique(NewChild);
	}

	void AddRange(TArray<FDebugRenderSceneProxy*> Children)
	{
		ChildProxies.Append(Children);
	}

private:
	uint32 GetAllocatedSizeInternal(void) const
	{
		SIZE_T Size = FDebugRenderSceneProxy::GetAllocatedSize() + ChildProxies.GetAllocatedSize();
		for (int32 Idx = 0; Idx < ChildProxies.Num(); Idx++)
		{
			Size += ChildProxies[Idx]->GetMemoryFootprint();
		}

		return IntCastChecked<uint32>(Size);
	}

protected:
	TArray<FDebugRenderSceneProxy*> ChildProxies;
};

void FGameplayDebuggerDebugDrawDelegateHelper::RegisterDebugDrawDelegateInternal()
{
	ensureMsgf(State != RegisteredState, TEXT("DrawDelegate is already Registered!"));
	if (State == InitializedState)
	{
		for (int32 Idx = 0; Idx < DebugDrawDelegateHelpers.Num(); Idx++)
		{
			DebugDrawDelegateHelpers[Idx]->RequestRegisterDebugDrawDelegate(nullptr /*Deferred Context*/);
		}
		State = RegisteredState;
	}
}

void FGameplayDebuggerDebugDrawDelegateHelper::UnregisterDebugDrawDelegate()
{
	ensureMsgf(State != InitializedState, TEXT("DrawDelegate is in an invalid State: %i !"), State);
	if (State == RegisteredState)
	{
		for (int32 Idx = 0; Idx < DebugDrawDelegateHelpers.Num(); Idx++)
		{
			DebugDrawDelegateHelpers[Idx]->UnregisterDebugDrawDelegate();
		}
		State = InitializedState;
	}
}

void FGameplayDebuggerDebugDrawDelegateHelper::Reset()
{
	for (int32 Idx = 0; Idx < DebugDrawDelegateHelpers.Num(); Idx++)
	{
		delete DebugDrawDelegateHelpers[Idx];
	}
	DebugDrawDelegateHelpers.Reset();
}

void FGameplayDebuggerDebugDrawDelegateHelper::AddDelegateHelper(FDebugDrawDelegateHelper* InDebugDrawDelegateHelper)
{
	check(InDebugDrawDelegateHelper);
	DebugDrawDelegateHelpers.Add(InDebugDrawDelegateHelper);
}

//////////////////////////////////////////////////////////////////////////
// UGameplayDebuggerRenderingComponent

UGameplayDebuggerRenderingComponent::UGameplayDebuggerRenderingComponent(const FObjectInitializer& ObjInitializer) : Super(ObjInitializer)
{
}

FDebugRenderSceneProxy* UGameplayDebuggerRenderingComponent::CreateDebugSceneProxy()
{
	GameplayDebuggerDebugDrawDelegateHelper.Reset();

	FGameplayDebuggerCompositeSceneProxy* CompositeProxy = nullptr;

	AGameplayDebuggerCategoryReplicator* OwnerReplicator = Cast<AGameplayDebuggerCategoryReplicator>(GetOwner());
	if (OwnerReplicator && OwnerReplicator->IsEnabled())
	{
		TArray<FDebugRenderSceneProxy*> SceneProxies;
		for (int32 Idx = 0; Idx < OwnerReplicator->GetNumCategories(); Idx++)
		{
			const TSharedRef<FGameplayDebuggerCategory> Category = OwnerReplicator->GetCategory(Idx);
			if (Category->IsCategoryEnabled())
			{
				FDebugDrawDelegateHelper* CategoryDelegateHelper = nullptr;
				FDebugRenderSceneProxy* CategorySceneProxy = Category->CreateDebugSceneProxy(this, CategoryDelegateHelper);
				if (CategorySceneProxy)
				{
					SceneProxies.Add(CategorySceneProxy);
				}

				if (CategoryDelegateHelper)
				{
					GameplayDebuggerDebugDrawDelegateHelper.AddDelegateHelper(CategoryDelegateHelper);
				}
			}
		}

		if (SceneProxies.Num())
		{
			CompositeProxy = new FGameplayDebuggerCompositeSceneProxy(this);
			CompositeProxy->AddRange(SceneProxies);
		}
	}

	return CompositeProxy;
}

FBoxSphereBounds UGameplayDebuggerRenderingComponent::CalcBounds(const FTransform &LocalToWorld) const
{
	return FBoxSphereBounds(FBox::BuildAABB(FVector::ZeroVector, FVector(1000000.0f, 1000000.0f, 1000000.0f)));
}

