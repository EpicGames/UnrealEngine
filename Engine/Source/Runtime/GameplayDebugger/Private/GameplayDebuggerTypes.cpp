// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayDebuggerTypes.h"
#include "InputCoreTypes.h"
#include "Misc/Compression.h"
#include "SceneView.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "GameplayDebuggerConfig.h"
#include "DrawDebugHelpers.h"
#include "CanvasItem.h"
#include "Containers/StringFwd.h"
#include "Engine/Canvas.h"
#include "Engine/World.h"
#include "GameFramework/PlayerController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayDebuggerTypes)

DEFINE_LOG_CATEGORY(LogGameplayDebug);

namespace FGameplayDebuggerUtils
{
	bool IsAuthority(UWorld* World)
	{
		return (World == nullptr) || (World->GetNetMode() != NM_Client) || World->IsPlayingReplay();
	}
}

namespace FGameplayDebuggerCanvasStrings
{
	FString ColorNameInput = "white";
	FString ColorNameEnabled = "green";
	FString ColorNameDisabled = "grey";
	FString ColorNameEnabledActiveRow = "green";
	FString ColorNameDisabledActiveRow = "black";

	FString Separator = "{white} | ";
	FString SeparatorSpace = "  ";
}

//////////////////////////////////////////////////////////////////////////
// FGameplayDebuggerShape

FGameplayDebuggerShape FGameplayDebuggerShape::MakePoint(const FVector& Location, const float Radius, const FColor& Color, const FString& Description)
{
	FGameplayDebuggerShape NewElement;
	NewElement.ShapeData.Add(Location);
	NewElement.ShapeData.Add(FVector(Radius, 0, 0));
	NewElement.Color = Color;
	NewElement.Description = Description;
	NewElement.Type = EGameplayDebuggerShape::Point;

	return NewElement;
}

FGameplayDebuggerShape FGameplayDebuggerShape::MakeSegment(const FVector& StartLocation, const FVector& EndLocation, const float Thickness, const FColor& Color, const FString& Description)
{
	FGameplayDebuggerShape NewElement;
	NewElement.ShapeData.Add(StartLocation);
	NewElement.ShapeData.Add(EndLocation);
	NewElement.ShapeData.Add(FVector(Thickness, 0, 0));
	NewElement.Color = Color;
	NewElement.Description = Description;
	NewElement.Type = EGameplayDebuggerShape::Segment;

	return NewElement;
}

FGameplayDebuggerShape FGameplayDebuggerShape::MakeSegment(const FVector& StartLocation, const FVector& EndLocation, const FColor& Color, const FString& Description)
{
	return MakeSegment(StartLocation, EndLocation, 1.0f, Color, Description);
}

FGameplayDebuggerShape FGameplayDebuggerShape::MakeArrow(const FVector& StartLocation, const FVector& EndLocation, const float HeadSize, const float Thickness, const FColor& Color, const FString& Description)
{
	FGameplayDebuggerShape NewElement;
	NewElement.ShapeData.Add(StartLocation);
	NewElement.ShapeData.Add(EndLocation);
	NewElement.ShapeData.Add(FVector(Thickness, HeadSize, 0));
	NewElement.Color = Color;
	NewElement.Description = Description;
	NewElement.Type = EGameplayDebuggerShape::Arrow;

	return NewElement;
}

FGameplayDebuggerShape FGameplayDebuggerShape::MakeBox(const FVector& Center, const FVector& Extent, const FColor& Color, const FString& Description)
{
	return MakeBox(Center, Extent, 1.0f, Color, Description);
}

FGameplayDebuggerShape FGameplayDebuggerShape::MakeBox(const FVector& Center, const FVector& Extent, const float Thickness, const FColor& Color, const FString& Description)
{
	FGameplayDebuggerShape NewElement;
	NewElement.ShapeData.Add(Center);
	NewElement.ShapeData.Add(Extent);
	NewElement.ShapeData.Add(FVector(Thickness, 0, 0));
	NewElement.Color = Color;
	NewElement.Description = Description;
	NewElement.Type = EGameplayDebuggerShape::Box;

	return NewElement;
}

FGameplayDebuggerShape FGameplayDebuggerShape::MakeBox(const FVector& Center, const FRotator& Rotation, const FVector& Extent, const FColor& Color, const FString& Description)
{
	return MakeBox(Center, Rotation, Extent, 1.0f, Color, Description);
}

FGameplayDebuggerShape FGameplayDebuggerShape::MakeBox(const FVector& Center, const FRotator& Rotation, const FVector& Extent, const float Thickness, const FColor& Color, const FString& Description)
{
	FGameplayDebuggerShape NewElement;
	NewElement.ShapeData.Add(Center);
	NewElement.ShapeData.Add(Extent);
	NewElement.ShapeData.Add(Rotation.Euler());
	NewElement.ShapeData.Add(FVector(Thickness, 0, 0));
	NewElement.Color = Color;
	NewElement.Description = Description;
	NewElement.Type = EGameplayDebuggerShape::Box;

	return NewElement;
}

FGameplayDebuggerShape FGameplayDebuggerShape::MakeCone(const FVector& Location, const FVector& Direction, const float Length, const FColor& Color, const FString& Description)
{
	FGameplayDebuggerShape NewElement;
	NewElement.ShapeData.Add(Location);
	NewElement.ShapeData.Add(Direction);
	NewElement.ShapeData.Add(FVector(Length, 0, 0));
	NewElement.Color = Color;
	NewElement.Description = Description;
	NewElement.Type = EGameplayDebuggerShape::Cone;

	return NewElement;
}

FGameplayDebuggerShape FGameplayDebuggerShape::MakeCylinder(const FVector& Center, const float Radius, const float HalfHeight, const FColor& Color, const FString& Description)
{
	FGameplayDebuggerShape NewElement;
	NewElement.ShapeData.Add(Center);
	NewElement.ShapeData.Add(FVector(Radius, 0, HalfHeight));
	NewElement.Color = Color;
	NewElement.Description = Description;
	NewElement.Type = EGameplayDebuggerShape::Cylinder;

	return NewElement;
}

FGameplayDebuggerShape FGameplayDebuggerShape::MakeCircle(const FVector& Center, const FVector& Up, const float Radius, const FColor& Color, const FString& Description)
{
	return MakeCircle(Center, Up, Radius, 1.f, Color, Description);
}

FGameplayDebuggerShape FGameplayDebuggerShape::MakeCircle(const FVector& Center, const FVector& Up, const float Radius, const float Thickness, const FColor& Color, const FString& Description)
{
	const FMatrix Axes = FRotationMatrix::MakeFromX(Up);
	
	FGameplayDebuggerShape NewElement;
	NewElement.ShapeData.Add(Center);
	NewElement.ShapeData.Add(Axes.GetUnitAxis(EAxis::Y));
	NewElement.ShapeData.Add(Axes.GetUnitAxis(EAxis::Z));
	NewElement.ShapeData.Add(FVector(Radius, Thickness, 0));
	NewElement.Color = Color;
	NewElement.Description = Description;
	NewElement.Type = EGameplayDebuggerShape::Circle;

	return NewElement;
}

FGameplayDebuggerShape FGameplayDebuggerShape::MakeCircle(const FVector& Center, const FVector& WidthAxis, const FVector& HeightAxis, const float Radius, const FColor& Color, const FString& Description)
{
	return MakeCircle(Center, WidthAxis, HeightAxis, Radius, 1.0f, Color, Description);
}

FGameplayDebuggerShape FGameplayDebuggerShape::MakeCircle(const FVector& Center, const FVector& WidthAxis, const FVector& HeightAxis, const float Radius, const float Thickness, const FColor& Color, const FString& Description)
{
	FGameplayDebuggerShape NewElement;
	NewElement.ShapeData.Add(Center);
	NewElement.ShapeData.Add(WidthAxis);
	NewElement.ShapeData.Add(HeightAxis);
	NewElement.ShapeData.Add(FVector(Radius, Thickness, 0));
	NewElement.Color = Color;
	NewElement.Description = Description;
	NewElement.Type = EGameplayDebuggerShape::Circle;

	return NewElement;
}

FGameplayDebuggerShape FGameplayDebuggerShape::MakeRectangle(const FVector& Center, const FVector& WidthAxis, const FVector& HeightAxis, const float Width, const float Height, const FColor& Color, const FString& Description)
{
	return MakeRectangle(Center, WidthAxis, HeightAxis, Width, Height, 1.0f, Color, Description);
}

FGameplayDebuggerShape FGameplayDebuggerShape::MakeRectangle(const FVector& Center, const FVector& WidthAxis, const FVector& HeightAxis, const float Width, const float Height, const float Thickness, const FColor& Color, const FString& Description)
{
	FGameplayDebuggerShape NewElement;
	NewElement.ShapeData.Add(Center);
	NewElement.ShapeData.Add(WidthAxis * Width * 0.5);
	NewElement.ShapeData.Add(HeightAxis * Height * 0.5);
	NewElement.ShapeData.Add(FVector(Thickness, 0, 0));
	NewElement.Color = Color;
	NewElement.Description = Description;
	NewElement.Type = EGameplayDebuggerShape::Rectangle;

	return NewElement;
}

FGameplayDebuggerShape FGameplayDebuggerShape::MakeCapsule(const FVector& Center, const float Radius, const float HalfHeight, const FColor& Color, const FString& Description)
{
	return MakeCapsule(Center, FRotator::ZeroRotator, Radius, HalfHeight, Color, Description);
}

FGameplayDebuggerShape FGameplayDebuggerShape::MakeCapsule(const FVector& Center, const FRotator& Rotation, const float Radius, const float HalfHeight, const FColor& Color, const FString& Description)
{
	FGameplayDebuggerShape NewElement;
	NewElement.ShapeData.Add(Center);
	NewElement.ShapeData.Add(FVector(Radius, 0, HalfHeight));
	NewElement.ShapeData.Add(Rotation.Euler());
	NewElement.Color = Color;
	NewElement.Description = Description;
	NewElement.Type = EGameplayDebuggerShape::Capsule;

	return NewElement;
}

FGameplayDebuggerShape FGameplayDebuggerShape::MakePolygon(TConstArrayView<FVector> Verts, const FColor& Color, const FString& Description)
{
	FGameplayDebuggerShape NewElement;
	NewElement.ShapeData = Verts;
	NewElement.Color = Color;
	NewElement.Description = Description;
	NewElement.Type = EGameplayDebuggerShape::Polygon;
	return NewElement;
}

FGameplayDebuggerShape FGameplayDebuggerShape::MakePolyline(const TConstArrayView<FVector> Verts, const FColor& Color, const FString& Description)
{
	return MakePolyline(Verts, 1.0f, Color, Description);
}

FGameplayDebuggerShape FGameplayDebuggerShape::MakePolyline(const TConstArrayView<FVector> Verts, const float Thickness, const FColor& Color, const FString& Description)
{
	FGameplayDebuggerShape NewElement;
	NewElement.ShapeData = Verts;
	NewElement.ShapeData.Add(FVector(Thickness, 0, 0));
	NewElement.Color = Color;
	NewElement.Description = Description;
	NewElement.Type = EGameplayDebuggerShape::Polyline;
	return NewElement;
}

FGameplayDebuggerShape FGameplayDebuggerShape::MakeSegmentList(TConstArrayView<FVector> Verts, const float Thickness, const FColor& Color, const FString& Description)
{
	FGameplayDebuggerShape NewElement;
	NewElement.ShapeData = Verts;
	NewElement.ShapeData.Add(FVector(Thickness, 0, 0));
	NewElement.Color = Color;
	NewElement.Description = Description;
	NewElement.Type = EGameplayDebuggerShape::Segment;
	return NewElement;
}

FGameplayDebuggerShape FGameplayDebuggerShape::MakeSegmentList(TConstArrayView<FVector> Verts, const FColor& Color, const FString& Description)
{
	return MakeSegmentList(Verts, 1.0f, Color, Description);
}

void FGameplayDebuggerShape::Draw(UWorld* World, FGameplayDebuggerCanvasContext& Context)
{
	constexpr bool bPersistent = false;
	constexpr float LifeTime = -1.0f;
	constexpr uint8 DepthPriority = SDPG_World;
	
	FVector DescLocation;
	switch (Type)
	{
	case EGameplayDebuggerShape::Point:
		if (ShapeData.Num() == 2 && ShapeData[1].X > 0)
		{
			DrawDebugSphere(World, ShapeData[0], static_cast<float>(ShapeData[1].X), 16, Color, bPersistent, LifeTime, DepthPriority);
			DescLocation = ShapeData[0];
		}
		break;

	case EGameplayDebuggerShape::Segment:
		if (ShapeData.Num() >= 3 && ShapeData.Last().X > 0)
		{
			const int32 NumVertices = (ShapeData.Num() - 1) & ~1; // Expect 2 vertices per line.
			const float Thickness = static_cast<float>(ShapeData.Last().X);
			DescLocation = FVector::ZeroVector;
			for (int32 Index = 0; Index < NumVertices; Index += 2)
			{
				DrawDebugLine(World, ShapeData[Index + 0], ShapeData[Index + 1], Color, bPersistent, LifeTime, DepthPriority, Thickness);
				DescLocation += ShapeData[Index + 0] + ShapeData[Index + 1];
			}
			if (NumVertices > 0)
			{
				DescLocation /= NumVertices; 
			}
		}
		break;

	case EGameplayDebuggerShape::Arrow:
		if (ShapeData.Num() == 3 && ShapeData[2].X > 0)
		{
			DrawDebugDirectionalArrow(World, ShapeData[0], ShapeData[1], ShapeData[2].Y, Color, bPersistent, LifeTime, DepthPriority, static_cast<float>(ShapeData[2].X));
			DescLocation = (ShapeData[0] + ShapeData[1]) * 0.5f;
		}
		break;

	case EGameplayDebuggerShape::Box:
		if (ShapeData.Num() == 3)
		{
			const float Thickness = ShapeData[2].X;
			DrawDebugBox(World, ShapeData[0], ShapeData[1], Color, bPersistent, LifeTime, DepthPriority, Thickness);
			DescLocation = ShapeData[0];
		}
		else if (ShapeData.Num() == 4)
		{
			const float Thickness = ShapeData[3].X;
			DrawDebugBox(World, ShapeData[0], ShapeData[1], FQuat::MakeFromEuler(ShapeData[2]), Color, bPersistent, LifeTime, DepthPriority, Thickness);
			DescLocation = ShapeData[0];
		}
		break;

	case EGameplayDebuggerShape::Cone:
		if (ShapeData.Num() == 3 && ShapeData[2].X > 0)
		{
			constexpr float DefaultConeAngle = 0.25f; // ~ 15 degrees
			DrawDebugCone(World, ShapeData[0], ShapeData[1], static_cast<float>(ShapeData[2].X), DefaultConeAngle, DefaultConeAngle, 16, Color, bPersistent, LifeTime, DepthPriority);
			DescLocation = ShapeData[0];
		}
		break;

	case EGameplayDebuggerShape::Cylinder:
		if (ShapeData.Num() == 2)
		{
			DrawDebugCylinder(World, ShapeData[0] - FVector(0, 0, ShapeData[1].Z), ShapeData[0] + FVector(0, 0, ShapeData[1].Z), static_cast<float>(ShapeData[1].X), 16, Color, bPersistent, LifeTime, DepthPriority);
			DescLocation = ShapeData[0];
		}
		break;

	case EGameplayDebuggerShape::Circle:
		if (ShapeData.Num() == 4)
		{
			DrawDebugCircle(World, ShapeData[0], static_cast<float>(ShapeData[3].X), 32, Color, bPersistent, LifeTime, DepthPriority, ShapeData[3].Y, ShapeData[1], ShapeData[2], false);
			DescLocation = ShapeData[0];
		}
		break;

	case EGameplayDebuggerShape::Capsule:
		if (ShapeData.Num() == 3)
		{
			DrawDebugCapsule(World, ShapeData[0], static_cast<float>(ShapeData[1].Z), static_cast<float>(ShapeData[1].X), FQuat::MakeFromEuler(ShapeData[2]), Color, bPersistent, LifeTime, DepthPriority);
			DescLocation = ShapeData[0];
		}
		break;

	case EGameplayDebuggerShape::Polyline:
		if (ShapeData.Num() >= 3 && ShapeData.Last().X > 0)
		{
			const int32 NumVertices = (ShapeData.Num() - 1);
			const FVector::FReal Thickness = ShapeData.Last().X;
			DescLocation = ShapeData[0];
			for (int32 Index = 0; Index < NumVertices - 1; Index++)
			{
				DrawDebugLine(World, ShapeData[Index + 0], ShapeData[Index + 1], Color, bPersistent, LifeTime, DepthPriority, Thickness);
				DescLocation += ShapeData[Index + 1];
			}
			if (NumVertices > 0)
			{
				DescLocation /= NumVertices; 
			}
		}
		break;
		
	case EGameplayDebuggerShape::Polygon:
		if (ShapeData.Num() > 0)
		{
			FVector MidPoint = FVector::ZeroVector;
			TArray<int32> Indices;
			for (int32 Idx = 0; Idx < ShapeData.Num(); Idx++)
			{
				Indices.Add(Idx);
				MidPoint += ShapeData[Idx];
			}

			DrawDebugMesh(World, ShapeData, Indices, Color, bPersistent, LifeTime, DepthPriority);
			DescLocation = MidPoint / ShapeData.Num();
		}
		break;

	case EGameplayDebuggerShape::Rectangle:
		if (ShapeData.Num() == 4)
		{
			const FVector& Center = ShapeData[0];
			const FVector& WidthAxis = ShapeData[1];
			const FVector& HeightAxis = ShapeData[2];
			const FVector P0 = Center - WidthAxis + HeightAxis;
			const FVector P1 = Center + WidthAxis + HeightAxis;
			const FVector P2 = Center + WidthAxis - HeightAxis;
			const FVector P3 = Center - WidthAxis - HeightAxis;
			const FVector::FReal Thickness = ShapeData[3].X;
			DrawDebugLine(World, P0, P1 , Color, bPersistent, LifeTime, DepthPriority, Thickness);
			DrawDebugLine(World, P1, P2 , Color, bPersistent, LifeTime, DepthPriority, Thickness);
			DrawDebugLine(World, P2, P3 , Color, bPersistent, LifeTime, DepthPriority, Thickness);
			DrawDebugLine(World, P3, P0 , Color, bPersistent, LifeTime, DepthPriority, Thickness);
		}
		break;
		
	default:
		break;
	}

	if (Description.Len() && Context.IsLocationVisible(DescLocation))
	{
		const FVector2D ScreenLoc = Context.ProjectLocation(DescLocation);
		Context.PrintAt(ScreenLoc.X, ScreenLoc.Y, Color, Description);
	}
}

FArchive& operator<<(FArchive& Ar, FGameplayDebuggerShape& Shape)
{
	Ar << Shape.ShapeData;
	Ar << Shape.Description;
	Ar << Shape.Color;

	uint8 TypeNum = static_cast<uint8>(Shape.Type);
	Ar << TypeNum;
	Shape.Type = static_cast<EGameplayDebuggerShape>(TypeNum);

	return Ar;
}

//////////////////////////////////////////////////////////////////////////
// FGameplayDebuggerCanvasContext

enum class EStringParserToken : uint8
{
	OpenTag,
	CloseTag,
	NewLine,
	EndOfString,
	RegularChar,
	Tab,
};

class FTaggedStringParser
{
public:
	struct FNode
	{
		FString String;
		FColor Color;
		bool bNewLine;

		FNode() : Color(FColor::White), bNewLine(false) {}
		FNode(const FColor& InColor) : Color(InColor), bNewLine(false) {}
	};

	TArray<FNode> NodeList;

	FTaggedStringParser(const FColor& InDefaultColor) : DefaultColor(InDefaultColor) {}

	void ParseString(const FString& StringToParse)
	{
		DataIndex = 0;
		DataString = StringToParse;
		if (DataIndex >= DataString.Len())
		{
			return;
		}

		const FString TabString("     ");
		FColor TagColor;
		FNode CurrentNode(DefaultColor);

		for (EStringParserToken Token = ReadToken(); Token != EStringParserToken::EndOfString; Token = ReadToken())
		{
			switch (Token)
			{
			case EStringParserToken::RegularChar:
				CurrentNode.String.AppendChar(DataString[DataIndex]);
				break;

			case EStringParserToken::NewLine:
				NodeList.Add(CurrentNode);
				CurrentNode = FNode(NodeList.Last().Color);
				CurrentNode.bNewLine = true;
				break;

			case EStringParserToken::Tab:
				CurrentNode.String.Append(TabString);
				break;

			case EStringParserToken::OpenTag:
				if (ParseTag(TagColor))
				{
					NodeList.Add(CurrentNode);
					CurrentNode = FNode(TagColor);
				}
				break;
			}

			DataIndex++;
		}

		NodeList.Add(CurrentNode);
	}

private:

	int32 DataIndex;
	FString DataString;
	FColor DefaultColor;

	EStringParserToken ReadToken() const
	{
		EStringParserToken OutToken = EStringParserToken::RegularChar;

		const TCHAR Char = DataIndex < DataString.Len() ? DataString[DataIndex] : TEXT('\0');
		switch (Char)
		{
		case TEXT('\0'):
			OutToken = EStringParserToken::EndOfString;
			break;
		case TEXT('{'):
			OutToken = EStringParserToken::OpenTag;
			break;
		case TEXT('}'):
			OutToken = EStringParserToken::CloseTag;
			break;
		case TEXT('\n'):
			OutToken = EStringParserToken::NewLine;
			break;
		case TEXT('\t'):
			OutToken = EStringParserToken::Tab;
			break;
		default:
			break;
		}

		return OutToken;
	}

	bool ParseTag(FColor& OutColor)
	{
		FString TagString;

		EStringParserToken Token = ReadToken();
		for (; Token != EStringParserToken::EndOfString && Token != EStringParserToken::CloseTag; Token = ReadToken())
		{
			if (Token == EStringParserToken::RegularChar)
			{
				TagString.AppendChar(DataString[DataIndex]);
			}
			
			DataIndex++;
		}

		bool bResult = false;
		if (Token == EStringParserToken::CloseTag)
		{
			const FString TagColorLower = TagString.ToLower();
			const bool bIsColorName = GColorList.IsValidColorName(*TagColorLower);
			
			if (bIsColorName)
			{
				OutColor = GColorList.GetFColorByName(*TagColorLower);
				bResult = true;
			}
			else
			{
				bResult = OutColor.InitFromString(TagString);
			}
		}

		return bResult;
	}
};

FGameplayDebuggerCanvasContext::FGameplayDebuggerCanvasContext(UCanvas* InCanvas, UFont* InFont)
{
	if (InCanvas)
	{
		Canvas = InCanvas;
		Font = InFont;
		CursorX = DefaultX = static_cast<float>(InCanvas->SafeZonePadX);
		CursorY = DefaultY = static_cast<float>(InCanvas->SafeZonePadY);
	}
	else
	{
		CursorX = DefaultX = 0.0f;
		CursorY = DefaultY = 0.0f;
	}
}

void FGameplayDebuggerCanvasContext::Print(const FString& String)
{
	Print(FColor::White, 1.0f, String);
}

void FGameplayDebuggerCanvasContext::Print(const FColor& Color, const FString& String)
{
	Print(Color,  1.0f, String);
}

void FGameplayDebuggerCanvasContext::Print(const FColor& Color, const float Alpha, const FString& String)
{
	FTaggedStringParser Parser(Color);
	Parser.ParseString(String);

	const float LineHeight = GetLineHeight();
	for (int32 NodeIdx = 0; NodeIdx < Parser.NodeList.Num(); NodeIdx++)
	{
		const FTaggedStringParser::FNode& NodeData = Parser.NodeList[NodeIdx];
		if (NodeData.bNewLine)
		{
			if (Canvas.IsValid() && (CursorY + LineHeight) > Canvas->ClipY)
			{
				DefaultX += Canvas->ClipX / 2;
				CursorY = 0.0f;
			}

			CursorX = DefaultX;
			CursorY += LineHeight;
		}

		if (NodeData.String.Len() > 0)
		{
			float SizeX = 0.0f, SizeY = 0.0f;
			MeasureString(NodeData.String, SizeX, SizeY);

			FLinearColor TextColor(NodeData.Color);
			TextColor.A = Alpha;
			
			FCanvasTextItem TextItem(FVector2D::ZeroVector, FText::FromString(NodeData.String), Font.Get(), TextColor);
			if (FontRenderInfo.bEnableShadow)
			{
				TextItem.EnableShadow(FColor::Black, FVector2D(1, 1));
			}

			DrawItem(TextItem, CursorX, CursorY);
			CursorX += SizeX;
		}
	}

	MoveToNewLine();
}

void FGameplayDebuggerCanvasContext::PrintAt(float PosX, float PosY, const FString& String)
{
	PrintAt(PosX, PosY, FColor::White, 1.0f, String);
}

void FGameplayDebuggerCanvasContext::PrintAt(float PosX, float PosY, const FColor& Color, const FString& String)
{
	PrintAt(PosX, PosY, Color, 1.0f, String);
}

void FGameplayDebuggerCanvasContext::PrintAt(float PosX, float PosY, const FColor& Color, const float Alpha, const FString& String)
{
	TGuardValue<float> ScopedCursorX(CursorX, PosX);
	TGuardValue<float> ScopedCursorY(CursorY, PosY);
	TGuardValue<float> ScopedDefaultX(DefaultX, PosX);
	TGuardValue<float> ScopedDefaultY(DefaultY, PosY);

	Print(Color, Alpha, String);
}

void FGameplayDebuggerCanvasContext::PrintfImpl(const FColor& Color, const float Alpha, const TCHAR* Fmt, ...)
{
	va_list Args;
	va_start(Args, Fmt);
	Print(Color, Alpha, TStringBuilder<512>().AppendV(Fmt, Args).ToString());
	va_end(Args);
}

void FGameplayDebuggerCanvasContext::PrintfAtImpl(float PosX, float PosY, const FColor& Color, const float Alpha, const TCHAR* Fmt, ...)
{
	va_list Args;
	va_start(Args, Fmt);
	PrintAt(PosX, PosY, Color, Alpha, TStringBuilder<512>().AppendV(Fmt, Args).ToString());
	va_end(Args);
}

void FGameplayDebuggerCanvasContext::MoveToNewLine()
{
	const float LineHeight = GetLineHeight();
	CursorY += LineHeight;
	CursorX = DefaultX;
}

void FGameplayDebuggerCanvasContext::MeasureString(const FString& String, float& OutSizeX, float& OutSizeY) const
{
	OutSizeX = OutSizeY = 0.0f;

	UCanvas* CanvasOb = Canvas.Get();
	if (CanvasOb)
	{
		FString StringWithoutFormatting = String;
		int32 BracketStart = INDEX_NONE;
		while (StringWithoutFormatting.FindChar(TEXT('{'), BracketStart))
		{
			int32 BracketEnd = INDEX_NONE;
			if (StringWithoutFormatting.FindChar(TEXT('}'), BracketEnd))
			{
				if (BracketEnd > BracketStart)
				{
					StringWithoutFormatting.RemoveAt(BracketStart, BracketEnd - BracketStart + 1, EAllowShrinking::No);
				}
			}
		}

		const float DPIScale = CanvasOb->GetDPIScale();
		TArray<FString> Lines;
		StringWithoutFormatting.ParseIntoArrayLines(Lines);
		
		UFont* FontOb = Font.Get();
		for (int32 Idx = 0; Idx < Lines.Num(); Idx++)
		{
			float LineSizeX = 0.0f, LineSizeY = 0.0f;
			CanvasOb->StrLen(FontOb, Lines[Idx], LineSizeX, LineSizeY, /* bDPIAwareStringMeasurement */ true);
			LineSizeX /= DPIScale;
			LineSizeY /= DPIScale;

			OutSizeX = FMath::Max(OutSizeX, LineSizeX);
			OutSizeY += LineSizeY;
		}
	}
}

float FGameplayDebuggerCanvasContext::GetLineHeight() const
{
	UFont* FontOb = Font.Get();
	return FontOb ? FontOb->GetMaxCharHeight() : 0.0f;
}

FVector2D FGameplayDebuggerCanvasContext::ProjectLocation(const FVector& Location) const
{
	UCanvas* CanvasOb = Canvas.Get();
	return CanvasOb ? FVector2D(CanvasOb->Project(Location)) : FVector2D::ZeroVector;
}

bool FGameplayDebuggerCanvasContext::IsLocationVisible(const FVector& Location) const
{
	return Canvas.IsValid() && Canvas->SceneView && Canvas->SceneView->ViewFrustum.IntersectSphere(Location, 1.0f);
}

void FGameplayDebuggerCanvasContext::DrawItem(FCanvasItem& Item, float PosX, float PosY)
{
	UCanvas* CanvasOb = Canvas.Get();
	if (CanvasOb)
	{
		CanvasOb->DrawItem(Item, PosX, PosY);
	}
}

void FGameplayDebuggerCanvasContext::DrawIcon(const FColor& Color, const FCanvasIcon& Icon, float PosX, float PosY, float Scale)
{
	UCanvas* CanvasOb = Canvas.Get();
	if (CanvasOb)
	{
		CanvasOb->SetDrawColor(Color);
		CanvasOb->DrawIcon(Icon, PosX, PosY, Scale);
	}
}

UWorld* FGameplayDebuggerCanvasContext::GetWorld() const 
{
	return World.IsValid() 
		? World.Get()
		: (PlayerController.IsValid() ? PlayerController->GetWorld() : nullptr);
}

//////////////////////////////////////////////////////////////////////////
// FGameplayDebuggerDataPack

int32 FGameplayDebuggerDataPack::PacketSize = 512;

bool FGameplayDebuggerDataPack::CheckDirtyAndUpdate()
{
	TArray<uint8> UncompressedBuffer;
	FMemoryWriter ArWriter(UncompressedBuffer);
	SerializeDelegate.Execute(ArWriter);

	const uint32 NewDataCRC = FCrc::MemCrc32(UncompressedBuffer.GetData(), UncompressedBuffer.Num());
	if ((NewDataCRC == DataCRC) && !bIsDirty)
	{
		return false;
	}

	DataCRC = NewDataCRC;
	return true;
}

bool FGameplayDebuggerDataPack::RequestReplication(int16 SyncCounter)
{
	if (bNeedsConfirmation && !bReceived)
	{
		return false;
	}

	TArray<uint8> UncompressedBuffer;
	FMemoryWriter ArWriter(UncompressedBuffer);
	SerializeDelegate.Execute(ArWriter);

	const uint32 NewDataCRC = FCrc::MemCrc32(UncompressedBuffer.GetData(), UncompressedBuffer.Num());
	if ((NewDataCRC == DataCRC) && !bIsDirty)
	{
		return false;
	}

	const int32 MaxUncompressedDataSize = PacketSize;

	Header.bIsCompressed = (UncompressedBuffer.Num() > MaxUncompressedDataSize);
	if (Header.bIsCompressed)
	{
		const int32 UncompressedSize = UncompressedBuffer.Num();

		int32 CompressionHeader = UncompressedSize;
		const int32 CompressionHeaderSize = sizeof(CompressionHeader);

		int32 CompressedSize = FMath::TruncToInt(1.1f * UncompressedSize);
		Data.SetNum(CompressionHeaderSize + CompressedSize);

		uint8* CompressedBuffer = Data.GetData();
		FMemory::Memcpy(CompressedBuffer, &CompressionHeader, CompressionHeaderSize);
		CompressedBuffer += CompressionHeaderSize;

		FCompression::CompressMemory(NAME_Zlib,	CompressedBuffer, CompressedSize, UncompressedBuffer.GetData(), UncompressedSize, COMPRESS_BiasMemory);

		Data.SetNum(CompressionHeaderSize + CompressedSize);
	}
	else
	{
		Data = UncompressedBuffer;
	}

	bNeedsConfirmation = IsMultiPacket(Data.Num());
	bReceived = false;
	bIsDirty = false;

	DataCRC = NewDataCRC;
	Header.DataOffset = 0;
	Header.DataSize = Data.Num();
	Header.SyncCounter = SyncCounter;
	Header.DataVersion++;
	return true;
}

void FGameplayDebuggerDataPack::OnReplicated()
{
	if (Header.DataSize == 0)
	{
		ResetDelegate.Execute();
		return;
	}

	if (Header.bIsCompressed)
	{
		uint8* CompressedBuffer = Data.GetData();
		int32 CompressionHeader = 0;
		const int32 CompressionHeaderSize = sizeof(CompressionHeader);

		FMemory::Memcpy(&CompressionHeader, CompressedBuffer, CompressionHeaderSize);
		CompressedBuffer += CompressionHeaderSize;

		const int32 CompressedSize = Data.Num() - CompressionHeaderSize;
		const int32 UncompressedSize = CompressionHeader;
		TArray<uint8> UncompressedBuffer;
		UncompressedBuffer.AddUninitialized(UncompressedSize);

		FCompression::UncompressMemory(NAME_Zlib, UncompressedBuffer.GetData(), UncompressedSize, CompressedBuffer, CompressedSize);

		FMemoryReader ArReader(UncompressedBuffer);
		SerializeDelegate.Execute(ArReader);
	}
	else
	{
		FMemoryReader ArReader(Data);
		SerializeDelegate.Execute(ArReader);
	}

	Header.DataOffset = Header.DataSize;
}

void FGameplayDebuggerDataPack::OnPacketRequest(int16 DataVersion, int32 DataOffset)
{
	// client should confirm with the same version and offset that server currently replicates
	if (DataVersion == Header.DataVersion && DataOffset == Header.DataOffset)
	{
		Header.DataOffset = FMath::Min(DataOffset + FGameplayDebuggerDataPack::PacketSize, Header.DataSize);
		bReceived = (Header.DataOffset == Header.DataSize);
	}
	// if for some reason it requests previous data version, rollback to first packet
	else if (DataVersion < Header.DataVersion)
	{
		Header.DataOffset = 0;
	}
	// it may also request a previous packet from the same version, rollback and send next one
	else if (DataVersion == Header.DataVersion && DataOffset < Header.DataOffset)
	{
		Header.DataOffset = FMath::Max(0, DataOffset + FGameplayDebuggerDataPack::PacketSize);
	}
}

//////////////////////////////////////////////////////////////////////////
// FGameplayDebuggerInputModifier

FGameplayDebuggerInputModifier FGameplayDebuggerInputModifier::Shift(true, false, false, false);
FGameplayDebuggerInputModifier FGameplayDebuggerInputModifier::Ctrl(false, true, false, false);
FGameplayDebuggerInputModifier FGameplayDebuggerInputModifier::Alt(false, false, true, false);
FGameplayDebuggerInputModifier FGameplayDebuggerInputModifier::Cmd(false, false, false, true);
FGameplayDebuggerInputModifier FGameplayDebuggerInputModifier::None;

//////////////////////////////////////////////////////////////////////////
// FGameplayDebuggerInputHandler

bool FGameplayDebuggerInputHandler::IsValid() const
{
	return FKey(KeyName).IsValid();
}

FString FGameplayDebuggerInputHandler::ToString() const
{
	FString KeyDesc = KeyName.ToString();
	
	if (Modifier.bShift)
	{
		KeyDesc = FString("Shift+") + KeyDesc;
	}

	if (Modifier.bAlt)
	{
		KeyDesc = FString("Alt+") + KeyDesc;
	}
	
	if (Modifier.bCtrl)
	{
		KeyDesc = FString("Ctrl+") + KeyDesc;
	}
	
	if (Modifier.bCmd)
	{
		KeyDesc = FString("Cmd+") + KeyDesc;
	}

	return KeyDesc;
}

//////////////////////////////////////////////////////////////////////////
// FGameplayDebuggerInputHandlerConfig

FName FGameplayDebuggerInputHandlerConfig::CurrentCategoryName;
FName FGameplayDebuggerInputHandlerConfig::CurrentExtensionName;

FGameplayDebuggerInputHandlerConfig::FGameplayDebuggerInputHandlerConfig(const FName ConfigName, const FName DefaultKeyName)
{
	KeyName = DefaultKeyName;
	UpdateConfig(ConfigName);
}

FGameplayDebuggerInputHandlerConfig::FGameplayDebuggerInputHandlerConfig(const FName ConfigName, const FName DefaultKeyName, const FGameplayDebuggerInputModifier& DefaultModifier)
{
	KeyName = DefaultKeyName;
	Modifier = DefaultModifier;
	UpdateConfig(ConfigName);
}

void FGameplayDebuggerInputHandlerConfig::UpdateConfig(const FName ConfigName)
{
	if (FGameplayDebuggerInputHandlerConfig::CurrentCategoryName != NAME_None)
	{
		UGameplayDebuggerConfig* MutableToolConfig = UGameplayDebuggerConfig::StaticClass()->GetDefaultObject<UGameplayDebuggerConfig>();
		MutableToolConfig->UpdateCategoryInputConfig(FGameplayDebuggerInputHandlerConfig::CurrentCategoryName, ConfigName, KeyName, Modifier);
	}
	else if (FGameplayDebuggerInputHandlerConfig::CurrentExtensionName != NAME_None)
	{
		UGameplayDebuggerConfig* MutableToolConfig = UGameplayDebuggerConfig::StaticClass()->GetDefaultObject<UGameplayDebuggerConfig>();
		MutableToolConfig->UpdateExtensionInputConfig(FGameplayDebuggerInputHandlerConfig::CurrentExtensionName, ConfigName, KeyName, Modifier);
	}
}
