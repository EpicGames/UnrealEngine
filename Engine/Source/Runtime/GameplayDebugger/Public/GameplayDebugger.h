// Copyright Epic Games, Inc. All Rights Reserved.

// GAMEPLAY DEBUGGER
// 
// This tool allows easy on screen debugging of gameplay data, supporting client-server replication.
// Data is organized into named categories, which can be toggled during debugging.
// 
// To enable it, press Apostrophe key (UGameplayDebuggerConfig::ActivationKey)
//
// Category class:
// - derives from FGameplayDebuggerCategory
// - implements at least CollectData() and DrawData() functions
// - If WITH_GAMEPLAY_DEBUGGER is defined (doesn't exist in shipping builds by default; see note below to override), it will include all default debug categories and menu
// - If WITH_GAMEPLAY_DEBUGGER_CORE is defined (see note below), it will compile the core parts of the tool, waiting for user to register a debug category and create a replicator when desired.
// - needs to be registered and unregistered manually by owning module
// - automatically replicate data added with FGameplayDebuggerCategory::AddTextLine, FGameplayDebuggerCategory::AddShape
// - automatically replicate data structs initialized with FGameplayDebuggerCategory::SetDataPackReplication
// - can define own input bindings (e.g. subcategories, etc)
//
// Extension class:
// - derives from FGameplayDebuggerExtension
// - needs to be registered and unregistered manually by owning module
// - can define own input bindings
// - basically it's a stateless, not replicated, not drawn category, ideal for making e.g. different actor selection mechanic
//
// 
// Check FGameplayDebuggerCategory_BehaviorTree for implementation example.
// Check AIModule/Private/AIModule.cpp for registration example.
//
// Note. Use 'SetupGameplayDebuggerSupport(Target)' when adding module to your project's Build.cs (see AIModule/AIModule.Build.cs)
// Note. Use 'bUseGameplayDebugger={0|1}' in your <ProjectTargetType>.Target.cs to force GameplayDebugger disabled/enabled (if enabled, it will implicitly set bUseGameplayDebuggerCore=1)
// Note. Use 'bUseGameplayDebuggerCore={0|1}' in your <ProjectTargetType>.Target.cs to compile on/off the GameplayDebugger core parts only
// 

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

enum class EGameplayDebuggerCategoryState : uint8
{
	EnabledInGameAndSimulate,
	EnabledInGame,
	EnabledInSimulate,
	Disabled,
	Hidden,
};

class IGameplayDebugger : public IModuleInterface
{

public:
	DECLARE_DELEGATE_RetVal(TSharedRef<class FGameplayDebuggerCategory>, FOnGetCategory);
	DECLARE_DELEGATE_RetVal(TSharedRef<class FGameplayDebuggerExtension>, FOnGetExtension);

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IGameplayDebugger& Get()
	{
		return FModuleManager::LoadModuleChecked< IGameplayDebugger >("GameplayDebugger");
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("GameplayDebugger");
	}

	virtual void RegisterCategory(FName CategoryName, FOnGetCategory MakeInstanceDelegate, EGameplayDebuggerCategoryState CategoryState = EGameplayDebuggerCategoryState::Disabled, int32 SlotIdx = INDEX_NONE) = 0;
	virtual void UnregisterCategory(FName CategoryName) = 0;
	virtual void NotifyCategoriesChanged() = 0;
	virtual void RegisterExtension(FName ExtensionName, FOnGetExtension MakeInstanceDelegate) = 0;
	virtual void UnregisterExtension(FName ExtensionName) = 0;
	virtual void NotifyExtensionsChanged() = 0;
};
