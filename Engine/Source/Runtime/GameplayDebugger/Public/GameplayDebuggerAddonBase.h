// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameplayDebuggerTypes.h"

class UWorld;
class AActor;
class APlayerController;
class AGameplayDebuggerCategoryReplicator;

class FGameplayDebuggerAddonBase : public TSharedFromThis<FGameplayDebuggerAddonBase>
{
public:

	virtual ~FGameplayDebuggerAddonBase() = default;

	int32 GetNumInputHandlers() const { return InputHandlers.Num(); }
	FGameplayDebuggerInputHandler& GetInputHandler(int32 HandlerId) { return InputHandlers[HandlerId]; }
	GAMEPLAYDEBUGGER_API FString GetInputHandlerDescription(int32 HandlerId) const;
	GAMEPLAYDEBUGGER_API UWorld* GetWorldFromReplicator() const;
	/** Returns the first non-null world associated with, in order: OwnerPC, DebugActor, AGameplayDebuggerCategoryReplicator owner */
	GAMEPLAYDEBUGGER_API UWorld* GetDataWorld(const APlayerController* OwnerPC, const AActor* DebugActor) const;

	/** [ALL] called when gameplay debugger is activated */
	GAMEPLAYDEBUGGER_API virtual void OnGameplayDebuggerActivated();

	/** [ALL] called when gameplay debugger is deactivated */
	GAMEPLAYDEBUGGER_API virtual void OnGameplayDebuggerDeactivated();

	/** check if simulate in editor mode is active */
	static GAMEPLAYDEBUGGER_API bool IsSimulateInEditor();

protected:

	/** tries to find selected actor in local world */
	GAMEPLAYDEBUGGER_API AActor* FindLocalDebugActor() const;

	/** returns replicator actor */
	GAMEPLAYDEBUGGER_API AGameplayDebuggerCategoryReplicator* GetReplicator() const;

	/** creates new key binding handler: single key press */
	template<class UserClass>
	bool BindKeyPress(FName KeyName, UserClass* KeyHandlerObject, typename FGameplayDebuggerInputHandler::FHandler::TMethodPtr< UserClass > KeyHandlerFunc, EGameplayDebuggerInputMode InputMode = EGameplayDebuggerInputMode::Local)
	{
		FGameplayDebuggerInputHandler NewHandler;
		NewHandler.KeyName = KeyName;
		NewHandler.Delegate.BindRaw(KeyHandlerObject, KeyHandlerFunc);
		NewHandler.Mode = InputMode;

		if (NewHandler.IsValid())
		{
			InputHandlers.Add(NewHandler);
			return true;
		}

		return false;
	}

	/** creates new key binding handler: key press with modifiers */
	template<class UserClass>
	bool BindKeyPress(FName KeyName, FGameplayDebuggerInputModifier KeyModifer, UserClass* KeyHandlerObject, typename FGameplayDebuggerInputHandler::FHandler::TMethodPtr< UserClass > KeyHandlerFunc, EGameplayDebuggerInputMode InputMode = EGameplayDebuggerInputMode::Local)
	{
		FGameplayDebuggerInputHandler NewHandler;
		NewHandler.KeyName = KeyName;
		NewHandler.Modifier = KeyModifer;
		NewHandler.Delegate.BindRaw(KeyHandlerObject, KeyHandlerFunc);
		NewHandler.Mode = InputMode;

		if (NewHandler.IsValid())
		{
			InputHandlers.Add(NewHandler);
			return true;
		}

		return false;
	}

	/** creates new key binding handler: customizable key press, stored in config files */
	template<class UserClass>
	bool BindKeyPress(const FGameplayDebuggerInputHandlerConfig& InputConfig, UserClass* KeyHandlerObject, typename FGameplayDebuggerInputHandler::FHandler::TMethodPtr< UserClass > KeyHandlerFunc, EGameplayDebuggerInputMode InputMode = EGameplayDebuggerInputMode::Local)
	{
		FGameplayDebuggerInputHandler NewHandler;
		NewHandler.KeyName = InputConfig.KeyName;
		NewHandler.Modifier = InputConfig.Modifier;
		NewHandler.Delegate.BindRaw(KeyHandlerObject, KeyHandlerFunc);
		NewHandler.Mode = InputMode;

		if (NewHandler.IsValid())
		{
			InputHandlers.Add(NewHandler);
			return true;
		}

		return false;
	}

private:

	friend class FGameplayDebuggerAddonManager;

	/** list of input handlers */
	TArray<FGameplayDebuggerInputHandler> InputHandlers;

	/** replicator actor */
	TWeakObjectPtr<AGameplayDebuggerCategoryReplicator> RepOwner;
};
