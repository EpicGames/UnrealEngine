// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameplayDebuggerCategory.h"
#include "GameplayDebugger.h"

class AGameplayDebuggerCategoryReplicator;
class FGameplayDebuggerExtension;

DECLARE_MULTICAST_DELEGATE(FOnGameplayDebuggerAddonEvent);

class FGameplayDebuggerCategory;
class AGameplayDebuggerCategoryReplicator;

struct FGameplayDebuggerCategoryInfo
{
	IGameplayDebugger::FOnGetCategory MakeInstanceDelegate;
	EGameplayDebuggerCategoryState DefaultCategoryState;
	EGameplayDebuggerCategoryState CategoryState;
	int32 SlotIdx;
};

struct FGameplayDebuggerExtensionInfo
{
	IGameplayDebugger::FOnGetExtension MakeInstanceDelegate;
	uint32 bDefaultEnabled : 1;
	uint32 bEnabled : 1;
};

class FGameplayDebuggerAddonManager
{
public:
	FGameplayDebuggerAddonManager();

	/** adds new category to managed collection */
	void RegisterCategory(FName CategoryName, IGameplayDebugger::FOnGetCategory MakeInstanceDelegate, EGameplayDebuggerCategoryState CategoryState, int32 SlotIdx);

	/** removes category from managed collection */
	void UnregisterCategory(FName CategoryName);

	/** notify about change in known categories */
	void NotifyCategoriesChanged();

	/** creates new category objects for all known types */
	void CreateCategories(AGameplayDebuggerCategoryReplicator& Owner, TArray<TSharedRef<FGameplayDebuggerCategory> >& CategoryObjects);

	/** adds new extension to managed collection */
	void RegisterExtension(FName ExtensionName, IGameplayDebugger::FOnGetExtension MakeInstanceDelegate);

	/** removes extension from managed collection */
	void UnregisterExtension(FName ExtensionName);

	/** notify about change in known extensions */
	void NotifyExtensionsChanged();

	/** creates new extension objects for all known types */
	void CreateExtensions(AGameplayDebuggerCategoryReplicator& Replicator, TArray<TSharedRef<FGameplayDebuggerExtension> >& ExtensionObjects);

	/** refresh category and extension data from config */
	void UpdateFromConfig();

	/** get slot-Id map */
	const TArray<TArray<int32> >& GetSlotMap() const { return SlotMap; }

	/** get slot-Name map */
	const TArray<FString>& GetSlotNames() const { return SlotNames; }

	/** get number of visible categories */
	int32 GetNumVisibleCategories() const { return NumVisibleCategories; }

	/** singleton accessor */
	static GAMEPLAYDEBUGGER_API FGameplayDebuggerAddonManager& GetCurrent();

	/** event called when CategoryMap changes */
	FOnGameplayDebuggerAddonEvent OnCategoriesChanged;

	/** event called when ExtensionMap changes */
	FOnGameplayDebuggerAddonEvent OnExtensionsChanged;

private:
	/** map of all known extensions indexed by their names */
	TMap<FName, FGameplayDebuggerExtensionInfo> ExtensionMap;

	/** map of all known categories indexed by their names */
	TMap<FName, FGameplayDebuggerCategoryInfo> CategoryMap;

	/** list of all slots and their categories Ids */
	TArray<TArray<int32> > SlotMap;

	/** list of slot names */
	TArray<FString> SlotNames;

	/** number of categories, excluding hidden ones */
	int32 NumVisibleCategories;
};
