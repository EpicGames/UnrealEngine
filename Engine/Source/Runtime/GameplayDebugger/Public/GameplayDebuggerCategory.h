// Copyright Epic Games, Inc. All Rights Reserved.


// GAMEPLAY DEBUGGER CATEGORY
// 
// Single category of gameplay debugger tool, responsible for gathering and presenting data.
// Category instances are created on both server and local sides, and can use replication to
// show server's state on client.
// 
// It should be compiled and used only when module is included, so every category class
// needs be placed in #if WITH_GAMEPLAY_DEBUGGER block (or WITH_GAMEPLAY_DEBUGGER_MENU when using WITH_GAMEPLAY_DEBUGGER_CORE).
// 
// 
// Server side category :
// - CollectData() function will be called in category having authority (server / standalone)
// - set CollectDataInterval for adding delay between data collection, default value is 0, meaning: every tick
// - AddTextLine() and AddShape() adds new data to replicate, both arrays are cleared before calling CollectData()
// - SetDataPackReplication() marks struct member variable for replication
// - MarkDataPackDirty() forces data pack replication, sometimes changes can go unnoticed (CRC based)
// 
// Local category :
// - DrawData() function will be called in every tick to present gathered data
// - everything added by AddTextLine() and AddShape() will be shown before calling DrawData()
// - CreateSceneProxy() allows creating custom scene proxies, use with MarkRenderStateDirty()
// - OnDataPackReplicated() notifies about receiving new data, use with MarkRenderStateDirty() if needed
// - BindKeyPress() allows creating custom key bindings active only when category is being displayed
// 
// 
// Categories needs to be manually registered and unregistered with GameplayDebugger.
// It's best to do it in owning module's Startup / Shutdown, similar to detail view customizations.
// Check AIModule.cpp for examples.

#pragma once

#include "CoreMinimal.h"
#include "GameplayDebuggerTypes.h"
#include "GameplayDebuggerAddonBase.h"

class AActor;
class APlayerController;
class FDebugRenderSceneProxy;
class UPrimitiveComponent;
struct FDebugDrawDelegateHelper;

/**
 * Single category of visual debugger tool
 */
class FGameplayDebuggerCategory : public FGameplayDebuggerAddonBase
{
public:

	GAMEPLAYDEBUGGER_API FGameplayDebuggerCategory();
	GAMEPLAYDEBUGGER_API virtual ~FGameplayDebuggerCategory();

	/** [AUTH] gather data for replication */
	GAMEPLAYDEBUGGER_API virtual void CollectData(APlayerController* OwnerPC, AActor* DebugActor);

	/** [LOCAL] draw collected data */
	GAMEPLAYDEBUGGER_API virtual void DrawData(APlayerController* OwnerPC, FGameplayDebuggerCanvasContext& CanvasContext);

	/** [LOCAL] creates a scene proxy for more advanced debug rendering */
	GAMEPLAYDEBUGGER_API virtual FDebugRenderSceneProxy* CreateDebugSceneProxy(const UPrimitiveComponent* InComponent, FDebugDrawDelegateHelper*& OutDelegateHelper);

	/** [LOCAL] called after successful replication of entire data pack to client */
	GAMEPLAYDEBUGGER_API virtual void OnDataPackReplicated(int32 DataPackId);

	/** [AUTH] adds line of text tagged with {color} to replicated data */
	GAMEPLAYDEBUGGER_API void AddTextLine(const FString& TextLine);

	/** [AUTH] adds shape to replicated data */
	GAMEPLAYDEBUGGER_API void AddShape(const FGameplayDebuggerShape& Shape);

	/** [LOCAL] draw category */
	GAMEPLAYDEBUGGER_API void DrawCategory(APlayerController* OwnerPC, FGameplayDebuggerCanvasContext& CanvasContext);

	/** [LOCAL] check if category should be drawn */
	bool ShouldDrawCategory(bool bHasDebugActor) const { return IsCategoryEnabled() && (!bShowOnlyWithDebugActor || bHasDebugActor); }

	/** [LOCAL] check if data pack replication status  */
	bool ShouldDrawReplicationStatus() const { return bShowDataPackReplication; }

	/** [ALL] get name of category */
	FName GetCategoryName() const { return CategoryName; }

	/** [ALL] check if category header should be drawn */
	bool IsCategoryHeaderVisible() const { return bShowCategoryName; }

	/** [ALL] check if category is enabled */
	bool IsCategoryEnabled() const { return bIsEnabled; }

	/** [ALL] check if category is local (present data) */
	bool IsCategoryLocal() const { return bIsLocal; }

	/** [ALL] check if category has authority (collects data) */
	bool IsCategoryAuth() const { return bHasAuthority; }

	/** [LOCAL] */
	bool ShouldCollectDataOnClient() const { return bAllowLocalDataCollection; }

	int32 GetNumDataPacks() const { return ReplicatedDataPacks.Num(); }
	float GetDataPackProgress(int32 DataPackId) const { return ReplicatedDataPacks.IsValidIndex(DataPackId) ? ReplicatedDataPacks[DataPackId].GetProgress() : 0.0f; }
	bool IsDataPackReplicating(int32 DataPackId) const { return ReplicatedDataPacks.IsValidIndex(DataPackId) && ReplicatedDataPacks[DataPackId].IsInProgress(); }
	FGameplayDebuggerDataPack::FHeader GetDataPackHeaderCopy(int32 DataPackId) const { return ReplicatedDataPacks.IsValidIndex(DataPackId) ? ReplicatedDataPacks[DataPackId].Header : FGameplayDebuggerDataPack::FHeader(); }

	// temporary functions for compatibility, will be removed soon
	TArray<FString> GetReplicatedLinesCopy() const { return ReplicatedLines; }
	TArray<FGameplayDebuggerShape> GetReplicatedShapesCopy() const { return ReplicatedShapes; }

protected:
	/**
	 * Returns view location and direction from replicated data if available or will extract it from
	 * the provided player controller. The replicated data is used first since it might be provided
	 * from a detached camera.
	 * @return whether OutViewLocation and OutViewDirection were assigned to valid values
	 * @note Method is expected to always return valid outputs if valid controller is provided. Otherwise it
	 * depends if the view information has been replicated before the methods gets called.
	 */
	GAMEPLAYDEBUGGER_API bool GetViewPoint(const APlayerController* OwnerPC, FVector& OutViewLocation, FVector& OutViewDirection) const;

	/**
	 * Indicates if a given location is within a vision cone built from provided view location and direction based
	 * on MaxViewDistance and MaxViewAngle from GameplayDebuggerUserSettings
	 */
	static GAMEPLAYDEBUGGER_API bool IsLocationInViewCone(const FVector& ViewLocation, const FVector& ViewDirection, const FVector& TargetLocation);

	/** [AUTH] marks data pack as needing replication */
	GAMEPLAYDEBUGGER_API void MarkDataPackDirty(int32 DataPackId);

	/** [LOCAL] requests new scene proxy */
	GAMEPLAYDEBUGGER_API void MarkRenderStateDirty();

	/** [LOCAL] preferred view flag for creating scene proxy */
	GAMEPLAYDEBUGGER_API FString GetSceneProxyViewFlag() const;

	/** [ALL] sets up DataPack replication, needs address of property holding data, DataPack's struct must define Serialize(FArchive& Ar) function
	 *  returns DataPackId
	 */
	template<typename T>
	int32 SetDataPackReplication(T* DataPackAddr, EGameplayDebuggerDataPack Flags = EGameplayDebuggerDataPack::ResetOnTick)
	{
		FGameplayDebuggerDataPack NewDataPack;
		NewDataPack.PackId = ReplicatedDataPacks.Num();
		NewDataPack.Flags = Flags;
		NewDataPack.SerializeDelegate.BindRaw(DataPackAddr, &T::Serialize);
		NewDataPack.ResetDelegate.BindLambda([=] { *DataPackAddr = T(); });

		ReplicatedDataPacks.Add(NewDataPack);
		return NewDataPack.PackId;
	}

	/** [AUTH] force data collection on next update */
	void ForceImmediateCollect() { LastCollectDataTime = -MAX_dbl; }

	/** [ALL] Clears out accumulated replicated data. */
	GAMEPLAYDEBUGGER_API void ResetReplicatedData();

	/** update interval, 0 = each tick */
	float CollectDataInterval;

	/** include data pack replication details in drawn messages */
	uint32 bShowDataPackReplication : 1;

	/** include remaining time to next data collection in drawn messages */
	uint32 bShowUpdateTimer : 1;

	/** include category name in drawn messages */
	uint32 bShowCategoryName : 1;

	/** draw category only when DebugActor is present */
	uint32 bShowOnlyWithDebugActor : 1;

	/** 
	  * False by default. If set to True will make data collection functions work on Local categories as well.
	  * Note that in client-server setting the data is still being replicated over from the server as well,
	  * and data collection on the client can also be used to annotate the incoming data. If this is undesired
	  * you need to call ResetReplicatedData before gathering the client-side data. 
	  */
	uint32 bAllowLocalDataCollection : 1;

private:

	friend class FGameplayDebuggerAddonManager;
	friend class AGameplayDebuggerCategoryReplicator;
	friend struct FGameplayDebuggerNetPack;

	/** if set, this category object can display data */
	uint32 bIsLocal : 1;

	/** if set, this category object can collect data */
	uint32 bHasAuthority : 1;

	/** if set, this category object is enabled in debugger */
	uint32 bIsEnabled : 1;

	/** Id number assigned to this category object */
	int32 CategoryId;

	/** timestamp of last update */
	double LastCollectDataTime;

	/** name of debugger category (auto assigned during category registration) */
	FName CategoryName;

	/** list of replicated text lines, will be reset before CollectData call on AUTH */
	TArray<FString> ReplicatedLines;

	/** list of replicated shapes, will be reset before CollectData call on AUTH */
	TArray<FGameplayDebuggerShape> ReplicatedShapes;

	/** list of replicated data packs */
	TArray<FGameplayDebuggerDataPack> ReplicatedDataPacks;
};
