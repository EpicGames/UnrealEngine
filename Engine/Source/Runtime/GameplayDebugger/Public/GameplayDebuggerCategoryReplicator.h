// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "GameFramework/Actor.h"
#include "GameplayDebuggerTypes.h"
#include "GameplayDebuggerCategoryReplicator.generated.h"

class AGameplayDebuggerCategoryReplicator;
class APlayerController;
class FGameplayDebuggerCategory;
class FGameplayDebuggerExtension;
class UGameplayDebuggerRenderingComponent;
class UPlayer;

// Struct used to send the DataPackPackets as RPC`s instead of using the CustomDeltaNetSerializer.
USTRUCT()
struct FGameplayDebuggerDataPackRPCParams
{
	GENERATED_BODY()

	UPROPERTY()
	FName CategoryName;

	UPROPERTY()
	int32 DataPackIdx = 0;

	UPROPERTY()
	FGameplayDebuggerDataPackHeader Header;

	UPROPERTY()
	TArray<uint8> Data;
};

USTRUCT()
struct FGameplayDebuggerCategoryData
{
	GENERATED_BODY()

	UPROPERTY()
	FName CategoryName;

	UPROPERTY()
	TArray<FString> TextLines;

	UPROPERTY()
	TArray<FGameplayDebuggerShape> Shapes;

	// Either replicated using the NetDeltaSerialize or alternatively as regular RPC`s
	UPROPERTY(NotReplicated)
	TArray<FGameplayDebuggerDataPackHeader> DataPacks;

	UPROPERTY()
	bool bIsEnabled = false;
};

USTRUCT()
struct FGameplayDebuggerNetPack
{
	GENERATED_BODY()

	UPROPERTY(NotReplicated)
	TObjectPtr<AGameplayDebuggerCategoryReplicator> Owner;

	FGameplayDebuggerNetPack() : Owner(nullptr) {}

	bool NetDeltaSerialize(FNetDeltaSerializeInfo & DeltaParms);
	void OnCategoriesChanged();

	// When using Iris we cannot use the same appoach as the current NetSerializeDelta as it does polling of data and also modifies data outside of the replicated state during serialization which is not allowed when using iris.
	// Therefore we need explicit methods to populate the state from the owner and to apply it to the owner which can be invoked before and after serialization as needed.
	void PopulateFromOwner();
	void ApplyToOwner();

private:
	UPROPERTY()
	TArray<FGameplayDebuggerCategoryData> SavedData;
};

template<>
struct TStructOpsTypeTraits<FGameplayDebuggerNetPack> : public TStructOpsTypeTraitsBase2<FGameplayDebuggerNetPack>
{
	enum
	{
		WithNetDeltaSerializer = true,
	};
};

USTRUCT()
struct FGameplayDebuggerDebugActor
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TWeakObjectPtr<AActor> Actor;

	UPROPERTY()
	FName ActorName;

	UPROPERTY()
	int16 SyncCounter = 0;
};

USTRUCT()
struct FGameplayDebuggerVisLogSync
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString DeviceIDs;
};

DECLARE_MULTICAST_DELEGATE_TwoParams(FNotifyGameplayDebuggerOwnerChange, AGameplayDebuggerCategoryReplicator*, APlayerController* /** Old Owner */);

UCLASS(NotBlueprintable, NotBlueprintType, notplaceable, noteditinlinenew, hidedropdown, Transient, MinimalAPI)
class AGameplayDebuggerCategoryReplicator : public AActor
{
	GENERATED_UCLASS_BODY()

	GAMEPLAYDEBUGGER_API virtual class UNetConnection* GetNetConnection() const override;
	GAMEPLAYDEBUGGER_API virtual bool IsNetRelevantFor(const AActor* RealViewer, const AActor* ViewTarget, const FVector& SrcLocation) const override;

protected:
	GAMEPLAYDEBUGGER_API virtual void BeginPlay() override;

	GAMEPLAYDEBUGGER_API virtual void OnReplicationStartedForIris(const FOnReplicationStartedParams&) override;

	GAMEPLAYDEBUGGER_API virtual const AActor* GetNetOwner() const override;
	GAMEPLAYDEBUGGER_API virtual UPlayer* GetNetOwningPlayer() override;
	GAMEPLAYDEBUGGER_API virtual UPlayer* GetNetOwningPlayerAnyRole() override;

public:
	GAMEPLAYDEBUGGER_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	GAMEPLAYDEBUGGER_API virtual void TickActor(float DeltaTime, enum ELevelTick TickType, FActorTickFunction& ThisTickFunction) override;
	GAMEPLAYDEBUGGER_API virtual void PostNetReceive() override;

	GAMEPLAYDEBUGGER_API virtual void PreReplication(IRepChangedPropertyTracker& ChangedPropertyTracker) override;

	/** [AUTH] set new owner */
	GAMEPLAYDEBUGGER_API void SetReplicatorOwner(APlayerController* InOwnerPC);
	static GAMEPLAYDEBUGGER_API FNotifyGameplayDebuggerOwnerChange NotifyDebuggerOwnerChange;

	/** [ALL] set replicator state */
	GAMEPLAYDEBUGGER_API void SetEnabled(bool bEnable);

	/** [ALL] set category state */
	GAMEPLAYDEBUGGER_API void SetCategoryEnabled(int32 CategoryId, bool bEnable);

	/** [ALL] set actor for debugging */
	GAMEPLAYDEBUGGER_API void SetDebugActor(AActor* Actor, bool bSelectInEditor = false);

	/** [ALL] set view matrix that should be used for culling and picking instead of the associated controller view point */
	GAMEPLAYDEBUGGER_API void SetViewPoint(const FVector& InViewLocation, const FVector& InViewDirection);

	/** Indicates that a view matrix has been set for culling and picking instead of the associated controller view point */
	bool IsViewPointSet() const { return ViewLocation.IsSet() && ViewDirection.IsSet(); }

	/** [ALL] reset view matrix that should be used for culling and picking to revert on the associated controller view point */
	GAMEPLAYDEBUGGER_API void ResetViewPoint();

	/** [ALL] send input event to category */
	GAMEPLAYDEBUGGER_API void SendCategoryInputEvent(int32 CategoryId, int32 HandlerId);

	/** [ALL] send input event to extension */
	GAMEPLAYDEBUGGER_API void SendExtensionInputEvent(int32 ExtensionId, int32 HandlerId);

	/** [AUTH] starts data collection */
	GAMEPLAYDEBUGGER_API void CollectCategoryData(bool bForce = false);

	/** get current debug actor */
	AActor* GetDebugActor() const { return DebugActor.Actor.IsStale() ? nullptr : DebugActor.Actor.Get(); }
	
	/** get name of debug actor */
	FName GetDebugActorName() const { return DebugActor.ActorName; }

	/** get sync counter, increased with every change of DebugActor */
	int16 GetDebugActorCounter() const { return DebugActor.SyncCounter; }

	/** get view point information */
	GAMEPLAYDEBUGGER_API bool GetViewPoint(FVector& OutViewLocation, FVector& OutViewDirection) const;

	const FGameplayDebuggerVisLogSync& GetVisLogSyncData() const { return VisLogSync; }

	/** get player controller owning this replicator */
	TObjectPtr<APlayerController> GetReplicationOwner() const { return OwnerPC; }

	/** get replicator state */
	bool IsEnabled() const { return bIsEnabled; }

	/** returns true if this replicator has been created for an editor world */
	bool IsEditorWorldReplicator() const { return bIsEditorWorldReplicator; }

	/** get category state */
	GAMEPLAYDEBUGGER_API bool IsCategoryEnabled(int32 CategoryId) const;

	/** check if debug actor was selected */
	bool HasDebugActor() const { return DebugActor.ActorName != NAME_None; }

	/** get category count */
	int32 GetNumCategories() const { return Categories.Num(); }

	/** get extension count */
	int32 GetNumExtensions() const { return Extensions.Num(); }

	/** get category object */
	TSharedRef<FGameplayDebuggerCategory> GetCategory(int32 CategoryId) const { return Categories[CategoryId]; }

	/** get category object */
	TSharedRef<FGameplayDebuggerExtension> GetExtension(int32 ExtensionId) const { return Extensions[ExtensionId]; }

	/** returns true if object was created for local player (client / standalone) */
	bool IsLocal() const { return bIsLocal; }

#if WITH_EDITOR
	GAMEPLAYDEBUGGER_API void InitForEditor();
#endif // WITH_EDITOR

protected:
	UFUNCTION()
	GAMEPLAYDEBUGGER_API void OnRep_ReplicatedData();


	friend FGameplayDebuggerNetPack;

	UPROPERTY(Replicated)
	TObjectPtr<APlayerController> OwnerPC;

	UPROPERTY(Replicated)
	bool bIsEnabled;

	UPROPERTY(Replicated, ReplicatedUsing=OnRep_ReplicatedData)
	FGameplayDebuggerNetPack ReplicatedData;

	UPROPERTY(Replicated)
	FGameplayDebuggerDebugActor	DebugActor;

	UPROPERTY(Replicated)
	FGameplayDebuggerVisLogSync	VisLogSync;

	/** rendering component needs to attached to some actor, and this is as good as any */
	UPROPERTY()
	TObjectPtr<UGameplayDebuggerRenderingComponent> RenderingComp;

	/** category objects */
	TArray<TSharedRef<FGameplayDebuggerCategory> > Categories;

	/** extension objects */
	TArray<TSharedRef<FGameplayDebuggerExtension> > Extensions;

	TOptional<FVector> ViewLocation;
	TOptional<FVector> ViewDirection;

	uint32 bIsEnabledLocal : 1;
	uint32 bHasAuthority : 1;
	uint32 bIsLocal : 1;
	uint32 bIsEditorWorldReplicator : 1;
	uint32 bSendDataPacksUsingRPC : 1;

	/** notify about changes in known category set */
	GAMEPLAYDEBUGGER_API void OnCategoriesChanged();

	/** notify about changes in known extensions set */
	GAMEPLAYDEBUGGER_API void OnExtensionsChanged();

	/** send notifies to all categories about current tool state */
	GAMEPLAYDEBUGGER_API void NotifyCategoriesToolState(bool bIsActive);

	/** send notifies to all categories about current tool state */
	GAMEPLAYDEBUGGER_API void NotifyExtensionsToolState(bool bIsActive);

	UFUNCTION(Server, Reliable, WithValidation, meta = (CallInEditor = "true"))
	GAMEPLAYDEBUGGER_API void ServerSetEnabled(bool bEnable);

	UFUNCTION(Server, Reliable, WithValidation, meta = (CallInEditor = "true"))
	GAMEPLAYDEBUGGER_API void ServerSetDebugActor(AActor* Actor, bool bSelectInEditor);

	UFUNCTION(Server, Reliable, WithValidation, meta = (CallInEditor = "true"))
	GAMEPLAYDEBUGGER_API void ServerSetViewPoint(const FVector& InViewLocation, const FVector& InViewDirection);

	UFUNCTION(Server, Reliable, WithValidation, meta = (CallInEditor = "true"))
	GAMEPLAYDEBUGGER_API void ServerResetViewPoint();

	UE_DEPRECATED(5.8, "ServerSetCategoryEnabled by CategoryId does not work when server/client mismatch indices, use ServerSetCategoryNameEnabled")
	UFUNCTION(Server, Reliable, WithValidation, meta = (CallInEditor = "true"))
	GAMEPLAYDEBUGGER_API void ServerSetCategoryEnabled(int32 CategoryId, bool bEnable);

	UFUNCTION(Server, Reliable, WithValidation, meta = (CallInEditor = "true"))
	GAMEPLAYDEBUGGER_API void ServerSetCategoryNameEnabled(FName CategoryName, bool bEnable);

	/** helper function for replicating input for category handlers */
	UE_DEPRECATED(5.8, "ServerSendCategoryInputEvent by CategoryId does not work when server/client mismatch indices, use ServerSendCategoryNameInputEvent")
	UFUNCTION(Server, Reliable, WithValidation, meta = (CallInEditor = "true"))
	GAMEPLAYDEBUGGER_API void ServerSendCategoryInputEvent(int32 CategoryId, int32 HandlerId);

	UFUNCTION(Server, Reliable, WithValidation, meta = (CallInEditor = "true"))
	GAMEPLAYDEBUGGER_API void ServerSendCategoryNameInputEvent(FName CategoryName, int32 HandlerId);

	/** helper function for replicating input for extension handlers */
	UFUNCTION(Server, Reliable, WithValidation, meta = (CallInEditor = "true"))
	GAMEPLAYDEBUGGER_API void ServerSendExtensionInputEvent(int32 ExtensionId, int32 HandlerId);

	/** helper function for optionally sending DataPackPackets as RPC's */
	UFUNCTION(Client, Reliable)
	GAMEPLAYDEBUGGER_API void ClientDataPackPacket(const FGameplayDebuggerDataPackRPCParams& Params);
	
	/** [LOCAL] notify from CategoryData replication */
	GAMEPLAYDEBUGGER_API void OnReceivedDataPackPacket(int32 CategoryId, int32 DataPackId, const FGameplayDebuggerDataPack& DataPacket);

	/** called both from BeginPlay and InitForEditor to setup instance's internal */
	GAMEPLAYDEBUGGER_API void Init();

private:
	GAMEPLAYDEBUGGER_API void SendDataPackPacket(FName CategoryName, int32 DataPackIdx, FGameplayDebuggerDataPack& DataPack);
};
