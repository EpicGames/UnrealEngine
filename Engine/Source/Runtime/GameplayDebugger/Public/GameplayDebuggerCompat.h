// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Bad?

#if WITH_EDITOR
#include "Editor/EditorEngine.h"
#include "LevelEditor.h"
#endif // WITH_EDITOR
#include "GameplayDebuggerAddonManager.h"
#include "GameplayDebuggerPlayerManager.h"

class AGameplayDebuggingReplicator;

class FGameplayDebuggerCompat : public FSelfRegisteringExec, public GameplayDebugger
{
public:
	// Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// End IModuleInterface

	void WorldAdded(UWorld* InWorld);
	void WorldDestroyed(UWorld* InWorld);
#if WITH_EDITOR
	void OnLevelActorAdded(AActor* InActor);
	void OnLevelActorDeleted(AActor* InActor);
	TSharedRef<FExtender> OnExtendLevelEditorViewMenu(const TSharedRef<FUICommandList> CommandList);
	void CreateSnappingOptionsMenu(FMenuBuilder& Builder);
	void CreateSettingSubMenu(FMenuBuilder& Builder);
	void HandleSettingChanged(FName PropertyName);
#endif

	TArray<TWeakObjectPtr<AGameplayDebuggingReplicator> >& GetAllReplicators(UWorld* InWorld);
	void AddReplicator(UWorld* InWorld, AGameplayDebuggingReplicator* InReplicator);
	void RemoveReplicator(UWorld* InWorld, AGameplayDebuggingReplicator* InReplicator);

	// Begin FExec Interface
	virtual bool Exec(UWorld* Inworld, const TCHAR* Cmd, FOutputDevice& Ar) override;
	// End FExec Interface

private:
	virtual bool CreateGameplayDebuggerForPlayerController(APlayerController* PlayerController) override;
	virtual bool IsGameplayDebuggerActiveForPlayerController(APlayerController* PlayerController) override;

	bool DoesGameplayDebuggingReplicatorExistForPlayerController(APlayerController* PlayerController);

	TMap<TWeakObjectPtr<UWorld>, TArray<TWeakObjectPtr<AGameplayDebuggingReplicator> > > AllReplicatorsPerWorlds;

public:
	virtual void RegisterCategory(FName CategoryName, FOnGetCategory MakeInstanceDelegate, EGameplayDebuggerCategoryState CategoryState = EGameplayDebuggerCategoryState::Disabled, int32 SlotIdx = INDEX_NONE);
	virtual void UnregisterCategory(FName CategoryName);
	virtual void NotifyCategoriesChanged();
	virtual void RegisterExtension(FName ExtensionName, IGameplayDebugger::FOnGetExtension MakeInstanceDelegate);
	virtual void UnregisterExtension(FName ExtensionName);
	virtual void NotifyExtensionsChanged();
	virtual void UseNewGameplayDebugger();

	void StartupNewDebugger();
	void ShutdownNewDebugger();
	AGameplayDebuggerPlayerManager& GetPlayerManager(UWorld* World);
	void OnWorldInitialized(UWorld* World, const UWorld::InitializationValues IVS);

	bool bNewDebuggerEnabled;
	FGameplayDebuggerAddonManager AddonManager;
	TMap<TWeakObjectPtr<UWorld>, TWeakObjectPtr<AGameplayDebuggerPlayerManager>> PlayerManagers;

private:

#if WITH_EDITOR
	FLevelEditorModule::FLevelEditorMenuExtender ViewMenuExtender;
#endif
};
