// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "HAL/Platform.h"
#include "InputCoreTypes.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "GameplayDebuggerConfig.generated.h"

class FArchive;
struct FGameplayDebuggerInputModifier;

UENUM()
enum class EGameplayDebuggerOverrideMode : uint8
{
	Enable,
	Disable,
	UseDefault,
};

USTRUCT()
struct FGameplayDebuggerInputConfig
{
	GENERATED_USTRUCT_BODY()

	FGameplayDebuggerInputConfig()
		: bModShift(false)
		, bModCtrl(false)
		, bModAlt(false)
		, bModCmd(false) {}

	UPROPERTY(VisibleAnywhere, Category = Input)
	FString ConfigName;

	UPROPERTY(EditAnywhere, Category = Input)
	FKey Key;

	UPROPERTY(EditAnywhere, Category = Input)
	uint32 bModShift : 1;

	UPROPERTY(EditAnywhere, Category = Input)
	uint32 bModCtrl : 1;

	UPROPERTY(EditAnywhere, Category = Input)
	uint32 bModAlt : 1;

	UPROPERTY(EditAnywhere, Category = Input)
	uint32 bModCmd : 1;
};

USTRUCT()
struct FGameplayDebuggerCategoryConfig
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(VisibleAnywhere, Category = Settings)
	FString CategoryName;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (EditCondition = "bOverrideSlotIdx", ClampMin = -1, ClampMax = 9, UIMin = -1, UIMax = 9))
	int32 SlotIdx;

	UPROPERTY(EditAnywhere, Category = Settings)
	EGameplayDebuggerOverrideMode ActiveInGame;

	UPROPERTY(EditAnywhere, Category = Settings)
	EGameplayDebuggerOverrideMode ActiveInSimulate;

	UPROPERTY(EditAnywhere, Category = Settings)
	EGameplayDebuggerOverrideMode Hidden;

	UPROPERTY(EditAnywhere, Category = Settings, AdvancedDisplay)
	uint32 bOverrideSlotIdx : 1;

	UPROPERTY(EditAnywhere, Category = Settings, EditFixedSize)
	TArray<FGameplayDebuggerInputConfig> InputHandlers;

	FGameplayDebuggerCategoryConfig()
		: SlotIdx(0)
		, ActiveInGame(EGameplayDebuggerOverrideMode::UseDefault)
		, ActiveInSimulate(EGameplayDebuggerOverrideMode::UseDefault)
		, Hidden(EGameplayDebuggerOverrideMode::UseDefault)
		, bOverrideSlotIdx(false)
	{}
};

USTRUCT()
struct FGameplayDebuggerExtensionConfig
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(VisibleAnywhere, Category = Settings)
	FString ExtensionName;

	UPROPERTY(EditAnywhere, Category = Settings)
	EGameplayDebuggerOverrideMode UseExtension;

	UPROPERTY(EditAnywhere, Category = Settings, EditFixedSize)
	TArray<FGameplayDebuggerInputConfig> InputHandlers;

	FGameplayDebuggerExtensionConfig() : UseExtension(EGameplayDebuggerOverrideMode::UseDefault) {}
};

UCLASS(config = Engine, defaultconfig, MinimalAPI)
class UGameplayDebuggerConfig : public UObject
{
	GENERATED_UCLASS_BODY()

	/** key used to activate visual debugger tool */
	UPROPERTY(config, EditAnywhere, Category = Input)
	FKey ActivationKey;

	/** select next category row */
	UPROPERTY(config, EditAnywhere, Category = Input)
	FKey CategoryRowNextKey;

	/** select previous category row */
	UPROPERTY(config, EditAnywhere, Category = Input)
	FKey CategoryRowPrevKey;

	/** select category slot 0 */
	UPROPERTY(config, EditAnywhere, Category = Input)
	FKey CategorySlot0;

	/** select category slot 1 */
	UPROPERTY(config, EditAnywhere, Category = Input)
	FKey CategorySlot1;

	/** select category slot 2 */
	UPROPERTY(config, EditAnywhere, Category = Input)
	FKey CategorySlot2;

	/** select category slot 3 */
	UPROPERTY(config, EditAnywhere, Category = Input)
	FKey CategorySlot3;

	/** select category slot 4 */
	UPROPERTY(config, EditAnywhere, Category = Input)
	FKey CategorySlot4;

	/** select category slot 5 */
	UPROPERTY(config, EditAnywhere, Category = Input)
	FKey CategorySlot5;

	/** select category slot 6 */
	UPROPERTY(config, EditAnywhere, Category = Input)
	FKey CategorySlot6;

	/** select category slot 7 */
	UPROPERTY(config, EditAnywhere, Category = Input)
	FKey CategorySlot7;

	/** select category slot 8 */
	UPROPERTY(config, EditAnywhere, Category = Input)
	FKey CategorySlot8;

	/** select category slot 9 */
	UPROPERTY(config, EditAnywhere, Category = Input)
	FKey CategorySlot9;

	/** additional canvas padding: left */
	UPROPERTY(config, EditAnywhere, Category = Display)
	float DebugCanvasPaddingLeft;

	/** additional canvas padding: right */
	UPROPERTY(config, EditAnywhere, Category = Display)
	float DebugCanvasPaddingRight;

	/** additional canvas padding: top */
	UPROPERTY(config, EditAnywhere, Category = Display)
	float DebugCanvasPaddingTop;

	/** additional canvas padding: bottom */
	UPROPERTY(config, EditAnywhere, Category = Display)
	float DebugCanvasPaddingBottom;

	/** enable text shadow by default */
	UPROPERTY(config, EditAnywhere, Category = Display)
	bool bDebugCanvasEnableTextShadow;

	UPROPERTY(config, EditAnywhere, Category = AddOns, EditFixedSize)
	TArray<FGameplayDebuggerCategoryConfig> Categories;

	UPROPERTY(config, EditAnywhere, Category = AddOns, EditFixedSize)
	TArray<FGameplayDebuggerExtensionConfig> Extensions;

	/** updates entry in Categories array and modifies category creation params */
	GAMEPLAYDEBUGGER_API void UpdateCategoryConfig(const FName CategoryName, int32& SlotIdx, uint8& CategoryState);

	/** updates entry in Categories array and modifies input binding params */
	GAMEPLAYDEBUGGER_API void UpdateCategoryInputConfig(const FName CategoryName, const FName InputName, FName& KeyName, FGameplayDebuggerInputModifier& KeyModifier);

	/** updates entry in Extensions array and modifies extension creation params */
	GAMEPLAYDEBUGGER_API void UpdateExtensionConfig(const FName ExtensionName, uint8& UseExtension);

	/** updates entry in Categories array and modifies input binding params */
	GAMEPLAYDEBUGGER_API void UpdateExtensionInputConfig(const FName ExtensionName, const FName InputName, FName& KeyName, FGameplayDebuggerInputModifier& KeyModifier);

	/** remove all category and extension data from unknown sources (outdated entries) */
	GAMEPLAYDEBUGGER_API void RemoveUnknownConfigs();

	GAMEPLAYDEBUGGER_API virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	GAMEPLAYDEBUGGER_API virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif

private:

	/** used for cleanup */
	TArray<FName> KnownCategoryNames;
	TArray<FName> KnownExtensionNames;
	TMultiMap<FName, FName> KnownCategoryInputNames;
	TMultiMap<FName, FName> KnownExtensionInputNames;
};


UCLASS(config = EditorPerProjectUserSettings, meta = (DisplayName = "Gameplay Debugger"), MinimalAPI)
class UGameplayDebuggerUserSettings : public UDeveloperSettings
{
	GENERATED_BODY()
protected:
	virtual FName GetCategoryName() const override { return TEXT("Advanced"); }

public:
	static int32 GetFontSize() { return GetDefault<UGameplayDebuggerUserSettings>()->FontSize; }
	static GAMEPLAYDEBUGGER_API void SetFontSize(const int32 InFontSize);

	/** Controls whether GameplayDebugger will be available in pure editor mode.
	 *  @Note that you need to reload the map for the changes to this property to take effect */
	UPROPERTY(config, EditAnywhere, Category = GameplayDebugger)
	uint32 bEnableGameplayDebuggerInEditor : 1;

	/**
	 * Distance from view location under which actors can be selected
	 * This distance can also be used by some categories to apply culling.
	 */
	UPROPERTY(config, EditAnywhere, Category = GameplayDebugger)
	float MaxViewDistance = 25000.0f;

	/**
	 * Angle from view direction under which actors can be selected
	 * This angle can also be used by some categories to apply culling.
	 */
	UPROPERTY(config, EditAnywhere, Category = GameplayDebugger, meta = (UIMin = 0, ClampMin = 0, UIMax = 180, ClampMax = 180, Units = deg))
	float MaxViewAngle = 45.f;

	/** 
	 * Persist which categories are enabled across editor sessions. Saved to disk as names since index is unstable. 
	 */
	UPROPERTY(config, EditAnywhere, Category = GameplayDebugger)
	TSet<FName> EnabledCategories;

protected:
	/** Font Size used by Gameplay Debugger */ 
	UPROPERTY(config, EditAnywhere, Category = GameplayDebugger)
	int32 FontSize = 10;
};
