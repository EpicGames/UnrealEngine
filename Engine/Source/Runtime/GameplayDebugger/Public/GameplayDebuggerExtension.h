// Copyright Epic Games, Inc. All Rights Reserved.

// GAMEPLAY DEBUGGER EXTENSION
// 
// Extensions allows creating additional key bindings for gameplay debugger.
// For example, you can use them to add another way of selecting actor to Debug.
//
// Replication is limited only to handling input events and tool state events,
// it's not possible to send variables or RPC calls
//
// It should be compiled and used only when module is included, so every extension class
// needs be placed in #if WITH_GAMEPLAY_DEBUGGER block.
// 
// Extensions needs to be manually registered and unregistered with GameplayDebugger.
// It's best to do it in owning module's Startup / Shutdown, similar to detail view customizations.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "GameplayDebuggerAddonBase.h"

class APlayerController;

class FGameplayDebuggerExtension : public FGameplayDebuggerAddonBase
{
public:

	virtual ~FGameplayDebuggerExtension() = default;
	GAMEPLAYDEBUGGER_API virtual void OnGameplayDebuggerActivated() override;
	GAMEPLAYDEBUGGER_API virtual void OnGameplayDebuggerDeactivated() override;

	/** [LOCAL] description for gameplay debugger's header row, newline character is ignored */
	GAMEPLAYDEBUGGER_API virtual FString GetDescription() const;

	/** [LOCAL] called when added to debugger tool or tool is activated */
	GAMEPLAYDEBUGGER_API virtual void OnActivated();

	/** [LOCAL] called when removed from debugger tool or tool is deactivated */
	GAMEPLAYDEBUGGER_API virtual void OnDeactivated();

	/** check if extension is created for local player */
	GAMEPLAYDEBUGGER_API bool IsLocal() const;

protected:

	/** get player controller owning gameplay debugger tool */
	GAMEPLAYDEBUGGER_API APlayerController* GetPlayerController() const;
};
