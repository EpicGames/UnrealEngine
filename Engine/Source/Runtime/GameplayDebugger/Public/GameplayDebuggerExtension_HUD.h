// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameplayDebuggerExtension.h"

class FGameplayDebuggerExtension_HUD : public FGameplayDebuggerExtension
{
public:

	FGameplayDebuggerExtension_HUD();

	virtual void OnActivated() override;
	virtual void OnDeactivated() override;
	virtual FString GetDescription() const override;

	static TSharedRef<FGameplayDebuggerExtension> MakeInstance();

protected:

	void ToggleGameHUD();
	void ToggleDebugMessages();
	void SetGameHUDEnabled(bool bEnable);
	void SetDebugMessagesEnabled(bool bEnable);

	uint32 bWantsHUDEnabled : 1;
	uint32 bIsGameHUDEnabled : 1;
	uint32 bAreDebugMessagesEnabled : 1;
	uint32 bPrevDebugMessagesEnabled : 1;
	mutable uint32 bIsCachedDescriptionValid : 1;

	int32 HudBindingIdx;
	int32 MessagesBindingIdx;

	mutable FString CachedDescription;
};
