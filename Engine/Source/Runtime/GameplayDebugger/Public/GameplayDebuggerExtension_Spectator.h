// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameplayDebuggerExtension.h"

class ADebugCameraController;

class FGameplayDebuggerExtension_Spectator : public FGameplayDebuggerExtension
{
public:

	FGameplayDebuggerExtension_Spectator();

	virtual void OnDeactivated() override;
	virtual FString GetDescription() const override;

	static TSharedRef<FGameplayDebuggerExtension> MakeInstance();

protected:

	void ToggleSpectatorMode();

	uint32 bHasInputBinding : 1;
	mutable uint32 bIsCachedDescriptionValid : 1;
	mutable FString CachedDescription;
	TWeakObjectPtr<ADebugCameraController> SpectatorController;
};
