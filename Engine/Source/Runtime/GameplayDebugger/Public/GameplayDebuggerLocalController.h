// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/EngineTypes.h"
#include "GameplayDebuggerLocalController.generated.h"

class AActor;
class AGameplayDebuggerCategoryReplicator;
class AGameplayDebuggerPlayerManager;
class FGameplayDebuggerCanvasContext;
class FGameplayDebuggerCategory;
class UInputComponent;
struct FKey;
class UFont;

UCLASS(NotBlueprintable, NotBlueprintType, noteditinlinenew, hidedropdown, Transient)
class UGameplayDebuggerLocalController : public UObject
{
	GENERATED_UCLASS_BODY()

	virtual void BeginDestroy() override;

	/** initialize controller with replicator owner */
	void Initialize(AGameplayDebuggerCategoryReplicator& Replicator, AGameplayDebuggerPlayerManager& Manager);

	/** remove from world */
	void Cleanup();

#if WITH_GAMEPLAY_DEBUGGER_MENU
	/** Whether the Debug Draw rendering is allowed for this replicator */
	void SetDebugDrawEnabled(const bool bInDebugDrawEnabled) { bDebugDrawEnabled = bInDebugDrawEnabled; }

	/** drawing event */
	void OnDebugDraw(class UCanvas* Canvas, class APlayerController* PC);

	/** binds input actions */
	void BindInput(UInputComponent& InputComponent);

	/** checks if key is bound by any action */
	bool GAMEPLAYDEBUGGER_API IsKeyBound(const FName KeyName) const;

	static void SetConsoleCommandsEnabled(const bool bEnabled) { bConsoleCommandsEnabled = bEnabled; }
#endif // WITH_GAMEPLAY_DEBUGGER_MENU	
protected:
	UPROPERTY()
	TObjectPtr<AGameplayDebuggerCategoryReplicator> CachedReplicator;

	UPROPERTY()
	TObjectPtr<AGameplayDebuggerPlayerManager> CachedPlayerManager;

	UPROPERTY()
	TObjectPtr<AActor> DebugActorCandidate;

	UPROPERTY()
	TObjectPtr<UFont> HUDFont;

	TArray<TArray<int32> > DataPackMap;
	TArray<TArray<int32> > SlotCategoryIds;
	TArray<FString> SlotNames;

	uint32 bSimulateMode : 1;
	uint32 bNeedsCleanup : 1;
	uint32 bIsSelectingActor : 1;
	uint32 bIsLocallyEnabled : 1;
	uint32 bPrevLocallyEnabled : 1;
	uint32 bEnableTextShadow : 1;
	uint32 bPrevScreenMessagesEnabled : 1;
#if WITH_EDITOR
	uint32 bActivateOnPIEEnd : 1;
#endif // WITH_EDITOR
	int32 NumCategorySlots;
	int32 NumCategories;

	static GAMEPLAYDEBUGGER_API bool bConsoleCommandsEnabled;

#if WITH_GAMEPLAY_DEBUGGER_MENU
	friend struct FGameplayDebuggerConsoleCommands;

	TSet<FName> UsedBindings;

	FString ActivationKeyDesc;
	FString RowUpKeyDesc;
	FString RowDownKeyDesc;
	FString CategoryKeysDesc;

	int32 ActiveRowIdx;
	static constexpr int32 NumCategoriesPerRow = 10;

	float PaddingLeft;
	float PaddingRight;
	float PaddingTop;
	float PaddingBottom;

	FTimerHandle StartSelectingActorHandle;
	FTimerHandle SelectActorTickHandle;

	bool bDebugDrawEnabled = true;

	void OnActivationPressed();
	void OnActivationPressedWithModifier();
	void OnActivationReleased();
	void OnActivationReleasedWithModifier();
	void OnCategory0Pressed();
	void OnCategory1Pressed();
	void OnCategory2Pressed();
	void OnCategory3Pressed();
	void OnCategory4Pressed();
	void OnCategory5Pressed();
	void OnCategory6Pressed();
	void OnCategory7Pressed();
	void OnCategory8Pressed();
	void OnCategory9Pressed();
	void OnCategoryRowUpPressed();
	void OnCategoryRowDownPressed();
	void OnCategoryBindingEvent(int32 CategoryId, int32 HandlerId);
	void OnExtensionBindingEvent(int32 ExtensionId, int32 HandlerId);

	enum class ESelectionMode : uint8
	{
		BestPawnCandidate,
		LocalPlayer
	};

	/** called short time after activation key was pressed and hold with 'Shift' */
	void OnStartSelectingLocalPlayer();

	/** sets the local player as the new debug actor */
	void OnSelectLocalPlayer();

	/** called short time after activation key was pressed and hold */
	void OnStartSelectingActor();

	/** called in tick during actor selection */
	void OnSelectActorTick();

	void OnStartSelecting(ESelectionMode SelectionMode);
	
	/** toggle state of categories in given slot */
	void ToggleSlotState(int32 SlotIdx);

	/** toggle debugger on/off */
	void ToggleActivation(ESelectionMode SelectionMode = ESelectionMode::BestPawnCandidate);

	/** draw header row */
	void DrawHeader(FGameplayDebuggerCanvasContext& CanvasContext);

	/** draw header for category */
	void DrawCategoryHeader(int32 CategoryId, TSharedRef<FGameplayDebuggerCategory> Category, FGameplayDebuggerCanvasContext& CanvasContext);

#if WITH_EDITOR
	/** event for simulate in editor mode */
	void OnSelectionChanged(UObject* Object);
	void OnSelectedObject(UObject* Object);
	void OnBeginPIE(const bool bIsSimulating);
	void OnEndPIE(const bool bIsSimulating);
#endif // WITH_EDITOR

	FString GetKeyDescriptionShort(const FKey& KeyBind) const;
	FString GetKeyDescriptionLong(const FKey& KeyBind) const;
	
	/** build DataPackMap for replication details */
	void RebuildDataPackMap();
#endif // WITH_GAMEPLAY_DEBUGGER_MENU

	/** called when known category set has changed */
	void OnCategoriesChanged();

	/** restore enabled categories from last session */
	void RestoreCategories();
};
