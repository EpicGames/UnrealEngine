// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Engine/World.h"
#include "GameplayDebugger.h"
#include "GameplayDebuggerAddonManager.h"

class AGameplayDebuggerPlayerManager;

DECLARE_MULTICAST_DELEGATE(FOnLocalControllerInitialized)
DECLARE_MULTICAST_DELEGATE(FOnLocalControllerUninitialized)
DECLARE_MULTICAST_DELEGATE(FOnDebuggerEdMode)

class FGameplayDebuggerModule : public IGameplayDebugger
{
public:	
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	virtual void RegisterCategory(FName CategoryName, IGameplayDebugger::FOnGetCategory MakeInstanceDelegate, EGameplayDebuggerCategoryState CategoryState, int32 SlotIdx) override;
	virtual void UnregisterCategory(FName CategoryName) override;
	virtual void NotifyCategoriesChanged() override;
	virtual void RegisterExtension(FName ExtensionName, IGameplayDebugger::FOnGetExtension MakeInstanceDelegate) override;
	virtual void UnregisterExtension(FName ExtensionName) override;
	virtual void NotifyExtensionsChanged() override;

	AGameplayDebuggerPlayerManager& GetPlayerManager(UWorld* World);
	void OnWorldInitialized(UWorld* World, const UWorld::InitializationValues IVS);
		
	FGameplayDebuggerAddonManager AddonManager;
	TMap<TWeakObjectPtr<UWorld>, TWeakObjectPtr<AGameplayDebuggerPlayerManager>> PlayerManagers;

	static GAMEPLAYDEBUGGER_API FOnLocalControllerInitialized OnLocalControllerInitialized;
	static GAMEPLAYDEBUGGER_API FOnLocalControllerUninitialized OnLocalControllerUninitialized;
#if WITH_EDITOR
	static GAMEPLAYDEBUGGER_API FOnDebuggerEdMode OnDebuggerEdModeActivation;
#endif // WITH_EDITOR
};
