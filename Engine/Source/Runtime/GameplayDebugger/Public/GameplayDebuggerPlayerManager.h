// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "Tickable.h"
#include "GameplayDebuggerPlayerManager.generated.h"

class AGameplayDebuggerCategoryReplicator;
class APlayerController;
class UGameplayDebuggerLocalController;
class UInputComponent;
class AGameModeBase;

USTRUCT()
struct FGameplayDebuggerPlayerData
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TObjectPtr<UGameplayDebuggerLocalController> Controller = nullptr;

	UPROPERTY()
	TObjectPtr<UInputComponent> InputComponent = nullptr;

	UPROPERTY()
	TObjectPtr<AGameplayDebuggerCategoryReplicator> Replicator = nullptr;
};

UCLASS(NotBlueprintable, NotBlueprintType, notplaceable, noteditinlinenew, hidedropdown, Transient, MinimalAPI)
class AGameplayDebuggerPlayerManager : public AActor, public FTickableGameObject
{
	GENERATED_UCLASS_BODY()

	GAMEPLAYDEBUGGER_API virtual void TickActor(float DeltaTime, enum ELevelTick TickType, FActorTickFunction& ThisTickFunction) override;

	// we're ticking only the manager only when in editor
	// FTickableGameObject begin
	virtual UWorld* GetTickableGameObjectWorld() const override { return GetWorld(); }
	GAMEPLAYDEBUGGER_API virtual void Tick(float DeltaTime) override;
	GAMEPLAYDEBUGGER_API virtual ETickableTickType GetTickableTickType() const override;
	GAMEPLAYDEBUGGER_API virtual TStatId GetStatId() const override;
	virtual bool IsTickableInEditor() const override { return true; }
	/** results in this instance not being ticked in any of the game worlds */
	virtual bool IsTickable() const override { return bEditorTimeTick; }
	// FTickableGameObject end

	GAMEPLAYDEBUGGER_API virtual void PostInitProperties() override;

protected:
	GAMEPLAYDEBUGGER_API virtual void BeginPlay() override;

	GAMEPLAYDEBUGGER_API void OnReplayScrubTeardown(UWorld* InWorld);
	GAMEPLAYDEBUGGER_API void OnGameModeLogout(AGameModeBase* GameMode, AController* Exiting);

public:
	GAMEPLAYDEBUGGER_API virtual void BeginDestroy() override;
	GAMEPLAYDEBUGGER_API virtual void EndPlay(const EEndPlayReason::Type Reason) override;
	GAMEPLAYDEBUGGER_API void Init();
		
	GAMEPLAYDEBUGGER_API void UpdateAuthReplicators();
	GAMEPLAYDEBUGGER_API void RegisterReplicator(AGameplayDebuggerCategoryReplicator& Replicator);
	GAMEPLAYDEBUGGER_API void RefreshInputBindings(AGameplayDebuggerCategoryReplicator& Replicator);

	GAMEPLAYDEBUGGER_API AGameplayDebuggerCategoryReplicator* GetReplicator(const APlayerController& OwnerPC) const;
	GAMEPLAYDEBUGGER_API UInputComponent* GetInputComponent(const APlayerController& OwnerPC) const;
	GAMEPLAYDEBUGGER_API UGameplayDebuggerLocalController* GetLocalController(const APlayerController& OwnerPC) const;
#if WITH_EDITOR
	UGameplayDebuggerLocalController* GetEditorController() const { return EditorWorldData.Controller; }
#endif // WITH_EDITOR
	
	GAMEPLAYDEBUGGER_API const FGameplayDebuggerPlayerData* GetPlayerData(const APlayerController& OwnerPC) const;

	static GAMEPLAYDEBUGGER_API AGameplayDebuggerPlayerManager& GetCurrent(UWorld* World);

	/** extracts view location and direction from a player controller that can be used for picking */
	static GAMEPLAYDEBUGGER_API void GetViewPoint(const APlayerController& OwnerPC, FVector& OutViewLocation, FVector& OutViewDirection);

protected:

	UPROPERTY()
	TArray<FGameplayDebuggerPlayerData> PlayerData;

	UPROPERTY()
	TArray<TObjectPtr<AGameplayDebuggerCategoryReplicator>> PendingRegistrations;

#if WITH_EDITORONLY_DATA 
	UPROPERTY()
	FGameplayDebuggerPlayerData EditorWorldData;
#endif // WITH_EDITORONLY_DATA

	uint8 bEditorTimeTick : 1;
	uint32 bHasAuthority : 1;
	uint32 bIsLocal : 1;
	uint32 bInitialized : 1;
};
