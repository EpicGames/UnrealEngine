// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Debug/DebugDrawComponent.h"
#include "DebugRenderSceneProxy.h"
#include "GameplayDebuggerRenderingComponent.generated.h"

class FGameplayDebuggerCompositeSceneProxy;

class FGameplayDebuggerDebugDrawDelegateHelper : public FDebugDrawDelegateHelper
{
	typedef FDebugDrawDelegateHelper Super;

public:
	virtual ~FGameplayDebuggerDebugDrawDelegateHelper() override
	{
		Reset();
	}

	void Reset();

	void AddDelegateHelper(FDebugDrawDelegateHelper* InDebugDrawDelegateHelper);

protected:
	virtual void RegisterDebugDrawDelegateInternal() override;
	virtual void UnregisterDebugDrawDelegate() override;

private:
	TArray<FDebugDrawDelegateHelper*> DebugDrawDelegateHelpers;
};



UCLASS(ClassGroup = Debug, NotBlueprintable, NotBlueprintType, noteditinlinenew, hidedropdown, Transient)
class UGameplayDebuggerRenderingComponent : public UDebugDrawComponent
{
	GENERATED_UCLASS_BODY()

protected:
	virtual FDebugRenderSceneProxy* CreateDebugSceneProxy() override;
	virtual FDebugDrawDelegateHelper& GetDebugDrawDelegateHelper() override { return GameplayDebuggerDebugDrawDelegateHelper; }
	virtual FBoxSphereBounds CalcBounds(const FTransform &LocalToWorld) const override;

private:
	FGameplayDebuggerDebugDrawDelegateHelper GameplayDebuggerDebugDrawDelegateHelper;
};
