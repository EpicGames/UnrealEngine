// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "Templates/IsArrayOrRefOfTypeByPredicate.h"
#include "Traits/IsCharEncodingCompatibleWith.h"
#include "UObject/ObjectMacros.h"
#include "GameplayDebuggerTypes.generated.h"

class FCanvasItem;
class UCanvas;
class UFont;
struct FCanvasIcon;
class APlayerController;
class UWorld;

DECLARE_LOG_CATEGORY_EXTERN(LogGameplayDebug, Log, All);

class FCanvasItem;
struct FCanvasIcon;

namespace FGameplayDebuggerUtils
{
	bool GAMEPLAYDEBUGGER_API IsAuthority(UWorld* World);
}

class FGameplayDebuggerCanvasContext
{
public:
	/** canvas used for drawing */
	TWeakObjectPtr<UCanvas> Canvas;

	/** current text font */
	TWeakObjectPtr<UFont> Font;

	/** the player controller associated with this debugger context  */
	TWeakObjectPtr<APlayerController> PlayerController;

	/** the world associated with this debugger context  */
	TWeakObjectPtr<UWorld> World;

	/** font render data */
	FFontRenderInfo FontRenderInfo;

	/** position of cursor */
	float CursorX, CursorY;

	/** default position of cursor */
	float DefaultX, DefaultY;

	FGameplayDebuggerCanvasContext() {}
	GAMEPLAYDEBUGGER_API FGameplayDebuggerCanvasContext(UCanvas* InCanvas, UFont* InFont);

	// print string on canvas
	GAMEPLAYDEBUGGER_API void Print(const FString& String);
	GAMEPLAYDEBUGGER_API void Print(const FColor& Color, const FString& String);
	GAMEPLAYDEBUGGER_API void Print(const FColor& Color, const float Alpha, const FString& String);
	GAMEPLAYDEBUGGER_API void PrintAt(float PosX, float PosY, const FString& String);
	GAMEPLAYDEBUGGER_API void PrintAt(float PosX, float PosY, const FColor& Color, const FString& String);
	GAMEPLAYDEBUGGER_API void PrintAt(float PosX, float PosY, const FColor& Color, const float Alpha, const FString& String);

	// print formatted string on canvas
	template <typename FmtType, typename... Types>
	void Printf(const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FGameplayDebuggerCanvasContext::Printf");

		PrintfImpl(FColor::White, 1.0f, (const TCHAR*)Fmt, Args...);
	}
	
	template <typename FmtType, typename... Types>
	void Printf(const FColor& Color, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FGameplayDebuggerCanvasContext::Printf");

		PrintfImpl(Color, 1.0f, (const TCHAR*)Fmt, Args...);
	}
	
	template <typename FmtType, typename... Types>
	void Printf(const FColor& Color, const float Alpha, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FGameplayDebuggerCanvasContext::Printf");

		PrintfImpl(Color, Alpha, (const TCHAR*)Fmt, Args...);
	}
	
	template <typename FmtType, typename... Types>
	void PrintfAt(float PosX, float PosY, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FGameplayDebuggerCanvasContext::PrintfAt");

		PrintfAtImpl(PosX, PosY, FColor::White, 1.0f, (const TCHAR*)Fmt, Args...);
	}
	
	template <typename FmtType, typename... Types>
	void PrintfAt(float PosX, float PosY, const FColor& Color, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FGameplayDebuggerCanvasContext::PrintfAt");

		PrintfAtImpl(PosX, PosY, Color, 1.0f, (const TCHAR*)Fmt, Args...);
	}
	
	template <typename FmtType, typename... Types>
	void PrintfAt(float PosX, float PosY, const FColor& Color, const float Alpha, const FmtType& Fmt, Types... Args)
	{
		static_assert(TIsArrayOrRefOfTypeByPredicate<FmtType, TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array.");
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to FGameplayDebuggerCanvasContext::PrintfAt");

		PrintfAtImpl(PosX, PosY, Color, Alpha, (const TCHAR*)Fmt, Args...);
	}

private:
	GAMEPLAYDEBUGGER_API void VARARGS PrintfImpl(const FColor& Color, const float Alpha, const TCHAR* Args, ...);
	GAMEPLAYDEBUGGER_API void VARARGS PrintfAtImpl(float PosX, float PosY, const FColor& Color, const float Alpha, const TCHAR* Args, ...);

public:
	// moves cursor to new line
	GAMEPLAYDEBUGGER_API void MoveToNewLine();

	// calculate size of string
	GAMEPLAYDEBUGGER_API void MeasureString(const FString& String, float& OutSizeX, float& OutSizeY) const;

	// get height of single line text
	GAMEPLAYDEBUGGER_API float GetLineHeight() const;

	// project world location on canvas
	GAMEPLAYDEBUGGER_API FVector2D ProjectLocation(const FVector& Location) const;

	// check if world location is visible in current view
	GAMEPLAYDEBUGGER_API bool IsLocationVisible(const FVector& Location) const;

	// draw item on canvas
	GAMEPLAYDEBUGGER_API void DrawItem(FCanvasItem& Item, float PosX, float PosY);

	// draw icon on canvas
	GAMEPLAYDEBUGGER_API void DrawIcon(const FColor& Color, const FCanvasIcon& Icon, float PosX, float PosY, float Scale = 1.f);

	// fetches the World associated with this context. Will use the World member variable, if set, and 
	// PlayerController's world otherwise. Note that it can still be null and needs to be tested.
	GAMEPLAYDEBUGGER_API UWorld* GetWorld() const;
};

namespace FGameplayDebuggerCanvasStrings
{
	extern GAMEPLAYDEBUGGER_API FString ColorNameInput;
	extern GAMEPLAYDEBUGGER_API FString ColorNameEnabled;
	extern GAMEPLAYDEBUGGER_API FString ColorNameDisabled;
	extern GAMEPLAYDEBUGGER_API FString ColorNameEnabledActiveRow;
	extern GAMEPLAYDEBUGGER_API FString ColorNameDisabledActiveRow;

	extern GAMEPLAYDEBUGGER_API FString Separator;
	extern GAMEPLAYDEBUGGER_API FString SeparatorSpace;
}

UENUM()
enum class EGameplayDebuggerShape : uint8
{
	Invalid,
	Point,
	Segment,
	Box,
	Cone,
	Cylinder,
	Circle,
	Rectangle,
	Capsule,
	Polygon,
	Polyline,
	Arrow,
};

USTRUCT()
struct FGameplayDebuggerShape
{
	GENERATED_BODY()

	/** points defining shape */
	UPROPERTY()
	TArray<FVector> ShapeData;

	/** description of shape */
	UPROPERTY()
	FString Description;

	/** color of shape */
	UPROPERTY()
	FColor Color;

	/** type of shape */
	UPROPERTY()
	EGameplayDebuggerShape Type;

	FGameplayDebuggerShape() : Color(EForceInit::ForceInit), Type(EGameplayDebuggerShape::Invalid) {}

	bool operator==(const FGameplayDebuggerShape& Other) const
	{
		return (Type == Other.Type) && (Color == Other.Color) && (Description == Other.Description) && (ShapeData == Other.ShapeData);
	}

	static GAMEPLAYDEBUGGER_API FGameplayDebuggerShape MakePoint(const FVector& Location, const float Radius, const FColor& Color, const FString& Description = FString());
	static GAMEPLAYDEBUGGER_API FGameplayDebuggerShape MakeSegment(const FVector& StartLocation, const FVector& EndLocation, const float Thickness, const FColor& Color, const FString& Description = FString());
	static GAMEPLAYDEBUGGER_API FGameplayDebuggerShape MakeSegment(const FVector& StartLocation, const FVector& EndLocation, const FColor& Color, const FString& Description = FString());
	static GAMEPLAYDEBUGGER_API FGameplayDebuggerShape MakeArrow(const FVector& StartLocation, const FVector& EndLocation, const float HeadSize, const float Thickness, const FColor& Color, const FString& Description = FString());
	static GAMEPLAYDEBUGGER_API FGameplayDebuggerShape MakeBox(const FVector& Center, const FVector& Extent, const FColor& Color, const FString& Description = FString());
	static GAMEPLAYDEBUGGER_API FGameplayDebuggerShape MakeBox(const FVector& Center, const FVector& Extent, const float Thickness, const FColor& Color, const FString& Description = FString());
	static GAMEPLAYDEBUGGER_API FGameplayDebuggerShape MakeBox(const FVector& Center, const FRotator& Rotation, const FVector& Extent, const FColor& Color, const FString& Description = FString());
	static GAMEPLAYDEBUGGER_API FGameplayDebuggerShape MakeBox(const FVector& Center, const FRotator& Rotation, const FVector& Extent, const float Thickness, const FColor& Color, const FString& Description = FString());
	static GAMEPLAYDEBUGGER_API FGameplayDebuggerShape MakeCone(const FVector& Location, const FVector& Direction, const float Length, const FColor& Color, const FString& Description = FString());
	static GAMEPLAYDEBUGGER_API FGameplayDebuggerShape MakeCylinder(const FVector& Center, const float Radius, const float HalfHeight, const FColor& Color, const FString& Description = FString());
	static GAMEPLAYDEBUGGER_API FGameplayDebuggerShape MakeCircle(const FVector& Center, const FVector& Up, const float Radius, const FColor& Color, const FString& Description = FString());
	static GAMEPLAYDEBUGGER_API FGameplayDebuggerShape MakeCircle(const FVector& Center, const FVector& Up, const float Radius, const float Thickness, const FColor& Color, const FString& Description = FString());
	static GAMEPLAYDEBUGGER_API FGameplayDebuggerShape MakeCircle(const FVector& Center, const FVector& WidthAxis, const FVector& HeightAxis, const float Radius, const FColor& Color, const FString& Description = FString());
	static GAMEPLAYDEBUGGER_API FGameplayDebuggerShape MakeCircle(const FVector& Center, const FVector& WidthAxis, const FVector& HeightAxis, const float Radius, const float Thickness, const FColor& Color, const FString& Description = FString());
	static GAMEPLAYDEBUGGER_API FGameplayDebuggerShape MakeRectangle(const FVector& Center, const FVector& WidthAxis, const FVector& HeightAxis, const float Width, const float Height, const FColor& Color, const FString& Description = FString());
	static GAMEPLAYDEBUGGER_API FGameplayDebuggerShape MakeRectangle(const FVector& Center, const FVector& WidthAxis, const FVector& HeightAxis, const float Width, const float Height, const float Thickness, const FColor& Color, const FString& Description = FString());
	static GAMEPLAYDEBUGGER_API FGameplayDebuggerShape MakeCapsule(const FVector& Center, const float Radius, const float HalfHeight, const FColor& Color, const FString& Description = FString());
	static GAMEPLAYDEBUGGER_API FGameplayDebuggerShape MakeCapsule(const FVector& Center, const FRotator& Rotation, const float Radius, const float HalfHeight, const FColor& Color, const FString& Description = FString());
	static GAMEPLAYDEBUGGER_API FGameplayDebuggerShape MakePolygon(TConstArrayView<FVector> Verts, const FColor& Color, const FString& Description = FString());
	static GAMEPLAYDEBUGGER_API FGameplayDebuggerShape MakePolyline(const TConstArrayView<FVector> Verts, const float Thickness, const FColor& Color, const FString& Description = FString());
	static GAMEPLAYDEBUGGER_API FGameplayDebuggerShape MakePolyline(const TConstArrayView<FVector> Verts, const FColor& Color, const FString& Description = FString());
	static GAMEPLAYDEBUGGER_API FGameplayDebuggerShape MakeSegmentList(const TConstArrayView<FVector> Verts, const float Thickness, const FColor& Color, const FString& Description = FString());
	static GAMEPLAYDEBUGGER_API FGameplayDebuggerShape MakeSegmentList(const TConstArrayView<FVector> Verts, const FColor& Color, const FString& Description = FString());

	GAMEPLAYDEBUGGER_API void Draw(UWorld* World, FGameplayDebuggerCanvasContext& Context);
};

FArchive& operator<<(FArchive& Ar, FGameplayDebuggerShape& Shape);

enum class EGameplayDebuggerDataPack : uint8
{
	Persistent,
	ResetOnActorChange,
	ResetOnTick,
};

USTRUCT()
struct FGameplayDebuggerDataPackHeader
{
	GENERATED_BODY()

	/** version, increased every time new Data is requested */
	UPROPERTY()
	int16 DataVersion;

	/** debug actor sync counter */
	UPROPERTY()
	int16 SyncCounter;

	/** size of Data array */
	UPROPERTY()
	int32 DataSize;

	/** offset to currently replicated portion of data */
	UPROPERTY()
	int32 DataOffset;

	/** is data compressed? */
	UPROPERTY()
	uint32 bIsCompressed : 1;

	FGameplayDebuggerDataPackHeader() : DataVersion(0), SyncCounter(0), DataSize(0), DataOffset(0), bIsCompressed(false) {}

	inline bool Equals(const FGameplayDebuggerDataPackHeader& Other) const
	{
		return (DataVersion == Other.DataVersion) && (DataSize == Other.DataSize) && (DataOffset == Other.DataOffset);
	}

	inline bool operator==(const FGameplayDebuggerDataPackHeader& Other) const
	{
		return Equals(Other);
	}

	inline bool operator!=(const FGameplayDebuggerDataPackHeader& Other) const
	{
		return !Equals(Other);
	}
};

struct FGameplayDebuggerDataPack
{
	using FHeader = FGameplayDebuggerDataPackHeader;

	DECLARE_DELEGATE(FOnReset);
	DECLARE_DELEGATE_OneParam(FOnSerialize, FArchive&);

	FGameplayDebuggerDataPack() :
		PackId(0), DataCRC(0), bIsDirty(false), bNeedsConfirmation(false), bReceived(false),
		Flags(EGameplayDebuggerDataPack::ResetOnTick)
	{}

	/** data being replicated */
	TArray<uint8> Data;

	/** minimal header used for DataPack state comparison */
	FHeader Header;

	/** auto assigned Id of pack */
	int32 PackId;

	/** CRC used to test changes in Data array */
	uint32 DataCRC;

	/** force net replication, regardless DataCRC */
	uint32 bIsDirty : 1;

	/** if set, replication must be confirmed by client before sending next update */
	uint32 bNeedsConfirmation : 1;

	/** set when client receives complete data pack */
	uint32 bReceived : 1;

	/** data pack flags */
	EGameplayDebuggerDataPack Flags;

	FOnReset ResetDelegate;
	FOnSerialize SerializeDelegate;

	static GAMEPLAYDEBUGGER_API int32 PacketSize;

	GAMEPLAYDEBUGGER_API bool CheckDirtyAndUpdate();
	GAMEPLAYDEBUGGER_API bool RequestReplication(int16 SyncCounter);

	GAMEPLAYDEBUGGER_API void OnReplicated();
	GAMEPLAYDEBUGGER_API void OnPacketRequest(int16 DataVersion, int32 DataOffset);

	/** get replication progress in (0..1) range */
	inline float GetProgress() const
	{
		return (Header.DataOffset != Header.DataSize) ? (static_cast<float>(Header.DataOffset) / static_cast<float>(Header.DataSize)) : 1.0f;
	}

	/** is replication in progress? */
	inline bool IsInProgress() const
	{
		return (Header.DataSize > 0) && (Header.DataOffset < Header.DataSize);
	}

	static bool IsMultiPacket(int32 TestSize)
	{
		return TestSize > PacketSize;
	}
};

enum class EGameplayDebuggerInputMode : uint8
{
	// input handler is called on local category
	Local,

	// input handler is replicated to authority category and called there
	Replicated,
};

struct FGameplayDebuggerInputModifier
{
	uint32 bShift : 1;
	uint32 bCtrl : 1;
	uint32 bAlt : 1;
	uint32 bCmd : 1;
	uint32 bPressed : 1;
	uint32 bReleased : 1;

	FGameplayDebuggerInputModifier() : bShift(false), bCtrl(false), bAlt(false), bCmd(false), bPressed(true), bReleased(false) {}
	FGameplayDebuggerInputModifier(bool bInShift, bool bInCtrl, bool bInAlt, bool bInCmd) : bShift(bInShift), bCtrl(bInCtrl), bAlt(bInAlt), bCmd(bInCmd), bPressed(true), bReleased(false) {}

	FGameplayDebuggerInputModifier operator+(const FGameplayDebuggerInputModifier& Other) const
	{
		return FGameplayDebuggerInputModifier(bShift || Other.bShift, bCtrl || Other.bCtrl, bAlt || Other.bAlt, bCmd || Other.bCmd);
	}

	static GAMEPLAYDEBUGGER_API FGameplayDebuggerInputModifier Shift;
	static GAMEPLAYDEBUGGER_API FGameplayDebuggerInputModifier Ctrl;
	static GAMEPLAYDEBUGGER_API FGameplayDebuggerInputModifier Alt;
	static GAMEPLAYDEBUGGER_API FGameplayDebuggerInputModifier Cmd;
	static GAMEPLAYDEBUGGER_API FGameplayDebuggerInputModifier None;
};

struct FGameplayDebuggerInputHandler
{
	DECLARE_DELEGATE(FHandler);

	FName KeyName;
	FGameplayDebuggerInputModifier Modifier;
	FHandler Delegate;
	EGameplayDebuggerInputMode Mode;

	FGameplayDebuggerInputHandler() : KeyName(NAME_None), Mode(EGameplayDebuggerInputMode::Local) {}

	GAMEPLAYDEBUGGER_API bool IsValid() const;
	GAMEPLAYDEBUGGER_API FString ToString() const;
};

/** 
 * Customizable key binding used by FGameplayDebuggerAddonBase (both categories and extensions)
 * Intended to use only from addon's constructor!
 * Check example in FGameplayDebuggerExtension_Spectator::FGameplayDebuggerExtension_Spectator() for details
 */
struct FGameplayDebuggerInputHandlerConfig
{
	FName KeyName;
	FGameplayDebuggerInputModifier Modifier;

	FGameplayDebuggerInputHandlerConfig() : KeyName(NAME_None) {}
	GAMEPLAYDEBUGGER_API FGameplayDebuggerInputHandlerConfig(const FName ConfigName, const FName DefaultKeyName);
	GAMEPLAYDEBUGGER_API FGameplayDebuggerInputHandlerConfig(const FName ConfigName, const FName DefaultKeyName, const FGameplayDebuggerInputModifier& DefaultModifier);

	static GAMEPLAYDEBUGGER_API FName CurrentCategoryName;
	static GAMEPLAYDEBUGGER_API FName CurrentExtensionName;

private:
	GAMEPLAYDEBUGGER_API void UpdateConfig(const FName ConfigName);
};
